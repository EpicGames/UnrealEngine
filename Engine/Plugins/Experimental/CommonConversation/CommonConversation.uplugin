{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Common Conversation",
	"Description": "An *experimental* plugin for authoring graph-based conversation trees",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "CommonConversationRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "CommonConversationGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "CommonConversationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GameFeatures",
			"Enabled": true
		}
	]
}