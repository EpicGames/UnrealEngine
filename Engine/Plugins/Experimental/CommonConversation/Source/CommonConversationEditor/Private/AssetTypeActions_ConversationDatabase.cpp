// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_ConversationDatabase.h"
#include "Framework/Application/SlateApplication.h"

#include "ConversationDatabase.h"
#include "ConversationEditor.h"

#include "SConversationDiff.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

uint32 FAssetTypeActions_ConversationDatabase::GetCategories()
{ 
	return EAssetTypeCategories::Gameplay;
}

void FAssetTypeActions_ConversationDatabase::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
	for (UObject* Object : InObjects)
	{
		if (UConversationDatabase* Bank = Cast<UConversationDatabase>(Object))
		{
			FConversationEditor::CreateConversationEditor(Mode, EditWithinLevelEditor, Bank);
		}
	}
}

UClass* FAssetTypeActions_ConversationDatabase::GetSupportedClass() const
{ 
	return UConversationDatabase::StaticClass();
}

void FAssetTypeActions_ConversationDatabase::PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const
{
	UConversationDatabase* OldBank = Cast<UConversationDatabase>(OldAsset);
	UConversationDatabase* NewBank = Cast<UConversationDatabase>(NewAsset);
	check(NewBank != nullptr || OldBank != nullptr);

	// sometimes we're comparing different revisions of one single asset (other 
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !NewBank || !OldBank || (NewBank->GetName() == OldBank->GetName());

	FString BankName;
	if (NewBank)
	{
		BankName = NewBank->GetName();
	}
	else if (OldBank)
	{
		BankName = OldBank->GetName();
	}
	const FText WindowTitle = bIsSingleAsset ?
		FText::Format(LOCTEXT("Conversation Diff", "{0} - Conversation Diff"), FText::FromString(BankName)) :
		LOCTEXT("NamelessConversationDiff", "Conversation Diff");

	const TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(1000,800));

	Window->SetContent(SNew(SConversationDiff)
		.OldBank(OldBank)
		.NewBank(NewBank)
		.OldRevision(OldRevision)
		.NewRevision(NewRevision)
		.ShowAssetNames(!bIsSingleAsset)
		.OpenInDefaults(const_cast<FAssetTypeActions_ConversationDatabase*>(this), &FAssetTypeActions_ConversationDatabase::OpenInDefaults) );

	// Make this window a child of the modal window if we've been spawned while one is active.
	TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveModalWindow();
	if ( ActiveModal.IsValid() )
	{
		FSlateApplication::Get().AddWindowAsNativeChild( Window.ToSharedRef(), ActiveModal.ToSharedRef() );
	}
	else
	{
		FSlateApplication::Get().AddWindow( Window.ToSharedRef() );
	}
}

void FAssetTypeActions_ConversationDatabase::OpenInDefaults(UConversationDatabase* OldBank, UConversationDatabase* NewBank) const
{
	const FString OldTextFilename = DumpAssetToTempFile(OldBank);
	const FString NewTextFilename = DumpAssetToTempFile(NewBank);

	// Get diff program to use
	const FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateDiffProcess(DiffCommand, OldTextFilename, NewTextFilename);
}

#undef LOCTEXT_NAMESPACE
