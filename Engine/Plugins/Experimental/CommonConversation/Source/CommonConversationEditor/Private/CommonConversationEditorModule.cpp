// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonConversationEditorModule.h"

#include "AssetTypeActions_ConversationDatabase.h"
#include "UObject/CoreRedirects.h"



#define LOCTEXT_NAMESPACE "CommonConversationEditor"

void FCommonConversationEditorModule::StartupModule()
{
	ItemDataAssetTypeActions.Add(MakeShared<FAssetTypeActions_ConversationDatabase>());

	IAssetTools& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	for (TSharedPtr<FAssetTypeActions_Base> AssetActionPtr : ItemDataAssetTypeActions)
	{
		AssetToolsModule.RegisterAssetTypeActions(AssetActionPtr.ToSharedRef());
	}

	TArray<FCoreRedirect> Redirects;
	Redirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/Script/CommonDialogueEditor"), TEXT("/Script/CommonConversationEditor"));
	FCoreRedirects::AddRedirectList(Redirects, TEXT("CommonConversationEditor"));
}

void FCommonConversationEditorModule::ShutdownModule()
{
	// Unregister the conversation bank item data asset type actions
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		for (auto& AssetTypeAction : ItemDataAssetTypeActions)
		{
			if (AssetTypeAction.IsValid())
			{
				AssetToolsModule.UnregisterAssetTypeActions(AssetTypeAction.ToSharedRef());
			}
		}
	}
	ItemDataAssetTypeActions.Empty();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCommonConversationEditorModule, CommonConversationEditor)
