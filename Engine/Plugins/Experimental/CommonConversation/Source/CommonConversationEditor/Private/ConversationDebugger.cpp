// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationDebugger.h"


#define USE_CONVERSATION_DEBUGGER 0

// #include "Conversation/BTNode.h"
// #include "Conversation/BTTaskNode.h"
// #include "Conversation/BTAuxiliaryNode.h"
//#include "ConversationGraphNode_CompositeDecorator.h"
#include "ConversationEditor.h"
#include "Editor/UnrealEdEngine.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"
//#include "ConversationGraphNode_Requirement.h"
//#include "ConversationGraphNode_Task.h"
//#include "Conversation/Conversation.h"
//#include "ConversationDelegates.h"
#include "Framework/Application/SlateApplication.h"
#include "ConversationDatabase.h"

FConversationDebugger::FConversationDebugger()
{
	TreeAsset = NULL;
	bIsPIEActive = false;
	bIsCurrentSubtree = false;
	StepForwardIntoIdx = INDEX_NONE;
	StepForwardOverIdx = INDEX_NONE;
	StepBackIntoIdx = INDEX_NONE;
	StepBackOverIdx = INDEX_NONE;
	StepOutIdx = INDEX_NONE;
	SavedTimestamp = 0.0f;
	CurrentTimestamp = 0.0f;

	FEditorDelegates::BeginPIE.AddRaw(this, &FConversationDebugger::OnBeginPIE);
	FEditorDelegates::EndPIE.AddRaw(this, &FConversationDebugger::OnEndPIE);
	FEditorDelegates::PausePIE.AddRaw(this, &FConversationDebugger::OnPausePIE);

#if USE_CONVERSATION_DEBUGGER
	UConversationComponent::ActiveDebuggerCounter++;
#endif
}

FConversationDebugger::~FConversationDebugger()
{
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::PausePIE.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
// 	FConversationDelegates::OnTreeStarted.RemoveAll(this);
// 	FConversationDelegates::OnDebugLocked.RemoveAll(this);
// 	FConversationDelegates::OnDebugSelected.RemoveAll(this);

#if USE_CONVERSATION_DEBUGGER
	UConversationComponent::ActiveDebuggerCounter--;
#endif
}

void FConversationDebugger::CacheRootNode()
{
// 	if(RootNode.IsValid() || TreeAsset == nullptr || TreeAsset->BTGraph == nullptr)
// 	{
// 		return;
// 	}
// 
// 	for (const auto& Node : TreeAsset->BTGraph->Nodes)
// 	{
// 		RootNode = Cast<UConversationGraphNode_Root>(Node);
// 		if (RootNode.IsValid())
// 		{
// 			break;
// 		}
// 	}
}

void FConversationDebugger::Setup(UConversationDatabase* InTreeAsset, TSharedRef<FConversationEditor> InEditorOwner)
{
	EditorOwner = InEditorOwner;
	TreeAsset = InTreeAsset;
	DebuggerInstanceIndex = INDEX_NONE;
	ActiveStepIndex = 0;
	LastValidStepId = INDEX_NONE;
	ActiveBreakpoints.Reset();
//	KnownInstances.Reset();

	CacheRootNode();

#if USE_CONVERSATION_DEBUGGER
	if (IsPIESimulating())
	{
		OnBeginPIE(GEditor->bIsSimulatingInEditor);

		Refresh();
	}
#endif
}

void FConversationDebugger::Refresh()
{
	CacheRootNode();

// 	if (IsPIESimulating() && IsDebuggerReady())
// 	{
// 		// make sure is grabs data if currently paused
// 		if (IsPlaySessionPaused() && TreeInstance.IsValid())
// 		{
// 			FindLockedDebugActor(GEditor->PlayWorld);
// 			
// 			UpdateDebuggerInstance();
// 			UpdateAvailableActions();
// 
// 			if (DebuggerInstanceIndex != INDEX_NONE)
// 			{
// 				UpdateDebuggerViewOnStepChange();
// 				UpdateDebuggerViewOnTick();
// 
// 				const FConversationDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];
// 				OnActiveNodeChanged(ShowInstance.ActivePath, HasContinuousPrevStep() ?
// 					TreeInstance->DebuggerSteps[ActiveStepIndex - 1].InstanceStack[DebuggerInstanceIndex].ActivePath :
// 					TArray<uint16>());
// 
// 				UpdateAssetFlags(ShowInstance, RootNode.Get(), ActiveStepIndex);
// 			}
// 		}
// 	}
}

void FConversationDebugger::Tick(float DeltaTime)
{
	if (TreeAsset == NULL || IsPlaySessionPaused())
	{
		return;
	}

// 	if (!TreeInstance.IsValid())
// 	{
// 		// clear state when active tree is lost
// 		if (DebuggerInstanceIndex != INDEX_NONE)
// 		{
// 			ClearDebuggerState();
// 		}
// 
// 		return;
// 	}

#if USE_CONVERSATION_DEBUGGER
	TArray<uint16> EmptyPath;

	int32 TestStepIndex = 0;
	for (int32 Idx = TreeInstance->DebuggerSteps.Num() - 1; Idx >= 0; Idx--)
	{
		const FConversationExecutionStep& Step = TreeInstance->DebuggerSteps[Idx];
		if (Step.StepIndex == LastValidStepId)
		{
			TestStepIndex = Idx;
			break;
		}
	}

	// find index of previously displayed state and notify about all changes in between to give breakpoints a chance to trigger
	for (int32 i = TestStepIndex; i < TreeInstance->DebuggerSteps.Num(); i++)
	{
		const FConversationExecutionStep& Step = TreeInstance->DebuggerSteps[i];
		if (Step.StepIndex > DisplayedStepIndex)
		{
			ActiveStepIndex = i;
			LastValidStepId = Step.StepIndex;

			UpdateDebuggerInstance();
			UpdateAvailableActions();

			if (DebuggerInstanceIndex != INDEX_NONE)
			{
				UpdateDebuggerViewOnStepChange();

				const FConversationDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];
				OnActiveNodeChanged(ShowInstance.ActivePath, HasContinuousPrevStep() ?
					TreeInstance->DebuggerSteps[ActiveStepIndex - 1].InstanceStack[DebuggerInstanceIndex].ActivePath :
					EmptyPath);
			}
		}

		// skip rest of them if breakpoint hit
		if (IsPlaySessionPaused())
		{
			break;
		}
	}

	UpdateDebuggerInstance();
	if (DebuggerInstanceIndex != INDEX_NONE)
	{
		const FConversationDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];

		if (DisplayedStepIndex != TreeInstance->DebuggerSteps[ActiveStepIndex].StepIndex)
		{
			UpdateAssetFlags(ShowInstance, RootNode.Get(), ActiveStepIndex);
		}

		// collect current runtime descriptions for every node
		TArray<FString> RuntimeDescriptions;
		TreeInstance->StoreDebuggerRuntimeValues(RuntimeDescriptions, ShowInstance.RootNode, DebuggerInstanceIndex);
	
		UpdateAssetRuntimeDescription(RuntimeDescriptions, RootNode.Get());
	}

	UpdateDebuggerViewOnTick();
#endif
}

bool FConversationDebugger::IsTickable() const
{
	return IsDebuggerReady();
}

void FConversationDebugger::OnBeginPIE(const bool bIsSimulating)
{
	bIsPIEActive = true;
	if(EditorOwner.IsValid())
	{
		TSharedPtr<FConversationEditor> EditorOwnerPtr = EditorOwner.Pin();
		EditorOwnerPtr->RegenerateMenusAndToolbars();
		//EditorOwnerPtr->DebuggerUpdateGraph();
	}

	ActiveBreakpoints.Reset();
	//CollectBreakpointsFromAsset(RootNode.Get());

	FindMatchingTreeInstance();

	// remove these delegates first as we can get multiple calls to OnBeginPIE()
	USelection::SelectObjectEvent.RemoveAll(this);
// 	FConversationDelegates::OnTreeStarted.RemoveAll(this);
// 	FConversationDelegates::OnDebugSelected.RemoveAll(this);

	USelection::SelectObjectEvent.AddRaw(this, &FConversationDebugger::OnObjectSelected);
// 	FConversationDelegates::OnTreeStarted.AddRaw(this, &FConversationDebugger::OnTreeStarted);
// 	FConversationDelegates::OnDebugSelected.AddRaw(this, &FConversationDebugger::OnAIDebugSelected);
}

void FConversationDebugger::OnEndPIE(const bool bIsSimulating)
{
	bIsPIEActive = false;
	if(EditorOwner.IsValid())
	{
		EditorOwner.Pin()->RegenerateMenusAndToolbars();
	}

	USelection::SelectObjectEvent.RemoveAll(this);
// 	FConversationDelegates::OnTreeStarted.RemoveAll(this);
// 	FConversationDelegates::OnDebugSelected.RemoveAll(this);

	ClearDebuggerState();
	ActiveBreakpoints.Reset();

//	FConversationDebuggerInstance EmptyData;
//	UpdateAssetFlags(EmptyData, RootNode.Get(), INDEX_NONE);
	UpdateDebuggerViewOnInstanceChange();
}

void FConversationDebugger::OnPausePIE(const bool bIsSimulating)
{
#if USE_CONVERSATION_DEBUGGER
	// We might have paused while executing a sub-tree, so make sure that the editor is showing the correct tree
	TSharedPtr<FConversationEditor> EditorOwnerPin = EditorOwner.Pin();
	if (EditorOwnerPin.IsValid() && TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		const FConversationExecutionStep& StepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];
		const int32 LastInstanceIndex = StepInfo.InstanceStack.Num() - 1;
		if (StepInfo.InstanceStack.IsValidIndex(LastInstanceIndex) && StepInfo.InstanceStack[LastInstanceIndex].TreeAsset != TreeAsset)
		{
			EditorOwnerPin->DebuggerSwitchAsset(StepInfo.InstanceStack[LastInstanceIndex].TreeAsset);
		}
	}
#endif
}

void FConversationDebugger::OnObjectSelected(UObject* Object)
{
// 	if (Object && Object->IsSelected())
// 	{
// 		UConversationComponent* InstanceComp = FindInstanceInActor(Cast<AActor>(Object));
// 		if (InstanceComp)
// 		{
// 			ClearDebuggerState();
// 			TreeInstance = InstanceComp;
// 
// 			UpdateDebuggerViewOnInstanceChange();
// 		}
// 	}
}

void FConversationDebugger::OnAIDebugSelected(const APawn* Pawn)
{
// 	UConversationComponent* TestComp = FindInstanceInActor((APawn*)Pawn);
// 	if (TestComp)
// 	{
// 		ClearDebuggerState();
// 		TreeInstance = TestComp;
// 
// 		UpdateDebuggerViewOnInstanceChange();
// 	}
}

//void FConversationDebugger::OnTreeStarted(const UConversationComponent& OwnerComp, const UConversation& InTreeAsset)
//{
// 	// start debugging if tree asset matches, and no other actor was selected
// 	if (!TreeInstance.IsValid() && TreeAsset && TreeAsset == &InTreeAsset)
// 	{
// 		ClearDebuggerState();
// 		TreeInstance = MakeWeakObjectPtr(const_cast<UConversationComponent*>(&OwnerComp));
// 
// 		UpdateDebuggerViewOnInstanceChange();
// 	}
// 
// 	// update known instances
// 	TWeakObjectPtr<UConversationComponent> KnownComp = const_cast<UConversationComponent*>(&OwnerComp);
// 	KnownInstances.AddUnique(KnownComp);
//}

void FConversationDebugger::ClearDebuggerState(bool bKeepSubtree)
{
// 	LastValidStepId = bKeepSubtree ? LastValidStepId : INDEX_NONE;
// 
// 	DebuggerInstanceIndex = INDEX_NONE;
// 	ActiveStepIndex = 0;
// 	DisplayedStepIndex = INDEX_NONE;
// 
// 	if (TreeAsset && RootNode.IsValid())
// 	{
// 		FConversationDebuggerInstance EmptyData;
// 		UpdateAssetFlags(EmptyData, RootNode.Get(), INDEX_NONE);
// 	}
}

void FConversationDebugger::UpdateDebuggerInstance()
{
#if 0
	int32 PrevStackIndex = DebuggerInstanceIndex;
	DebuggerInstanceIndex = INDEX_NONE;

	if (TreeInstance.IsValid())
	{
#if USE_CONVERSATION_DEBUGGER
		if (TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
		{
			const FConversationExecutionStep& StepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];
			for (int32 i = 0; i < StepInfo.InstanceStack.Num(); i++)
			{
				if (StepInfo.InstanceStack[i].TreeAsset == TreeAsset)
				{
					DebuggerInstanceIndex = i;
					break;
				}
			}
		}
#endif
		UpdateCurrentSubtree();
	}

	if (DebuggerInstanceIndex != PrevStackIndex)
	{
		UpdateDebuggerViewOnInstanceChange();
	}
#endif
}

// void FConversationDebugger::SetNodeFlags(const struct FConversationDebuggerInstance& Data, class UConversationGraphNode* Node, class UBTNode* NodeInstance)
// {
// 	const bool bIsNodeActivePath = Data.ActivePath.Contains(NodeInstance->GetExecutionIndex());
// 	const bool bIsNodeActiveAdditional = Data.AdditionalActiveNodes.Contains(NodeInstance->GetExecutionIndex());
// 	const bool bIsNodeActive = bIsNodeActivePath || bIsNodeActiveAdditional;
// 	const bool bIsShowingCurrentState = IsShowingCurrentState();
// 
// 	Node->DebuggerUpdateCounter = DisplayedStepIndex;
// 	Node->bDebuggerMarkCurrentlyActive = bIsNodeActive && bIsShowingCurrentState;
// 	Node->bDebuggerMarkPreviouslyActive = bIsNodeActive && !bIsShowingCurrentState;
// 	
// 	const bool bIsTaskNode = NodeInstance->IsA(UBTTaskNode::StaticClass());
// 	Node->bDebuggerMarkFlashActive = bIsNodeActivePath && bIsTaskNode && IsPlaySessionRunning();
// 	Node->bDebuggerMarkSearchTrigger = false;
// 	Node->bDebuggerMarkSearchFailedTrigger = false;
// 
// 	Node->bDebuggerMarkBreakpointTrigger = NodeInstance->GetExecutionIndex() == StoppedOnBreakpointExecutionIndex;
// 	if (Node->bDebuggerMarkBreakpointTrigger)
// 	{
// 		if(EditorOwner.IsValid())
// 		{
// 			EditorOwner.Pin()->JumpToNode(Node);
// 		}
// 	}
// 
// 	int32 SearchPathIdx = INDEX_NONE;
// 	int32 NumTriggers = 0;
// 	bool bTriggerOnly = false;
// 
// 	for (int32 i = 0; i < Data.PathFromPrevious.Num(); i++)
// 	{
// 		const FConversationDebuggerInstance::FNodeFlowData& SearchStep = Data.PathFromPrevious[i];
// 		const bool bMatchesNodeIndex = (SearchStep.ExecutionIndex == NodeInstance->GetExecutionIndex());
// 		if (SearchStep.bTrigger || SearchStep.bDiscardedTrigger)
// 		{
// 			NumTriggers++;
// 			if (bMatchesNodeIndex)
// 			{
// 				Node->bDebuggerMarkSearchTrigger = SearchStep.bTrigger;
// 				Node->bDebuggerMarkSearchFailedTrigger = SearchStep.bDiscardedTrigger;
// 				bTriggerOnly = true;
// 			}
// 		}
// 		else if (bMatchesNodeIndex)
// 		{
// 			SearchPathIdx = i;
// 			bTriggerOnly = false;
// 		}
// 	}
// 
// 	Node->bDebuggerMarkSearchSucceeded = (SearchPathIdx != INDEX_NONE) && Data.PathFromPrevious[SearchPathIdx].bPassed;
// 	Node->bDebuggerMarkSearchFailed = (SearchPathIdx != INDEX_NONE) && !Data.PathFromPrevious[SearchPathIdx].bPassed;
// 	Node->DebuggerSearchPathIndex = bTriggerOnly ? 0 : FMath::Max(-1, SearchPathIdx - NumTriggers);
// 	Node->DebuggerSearchPathSize = Data.PathFromPrevious.Num() - NumTriggers;
// }

void FConversationDebugger::SetCompositeDecoratorFlags(const struct FConversationDebuggerInstance& Data, class UConversationGraphNode_CompositeDecorator* Node)
{
// 	const bool bIsShowingCurrentState = IsShowingCurrentState();
// 	bool bIsNodeActive = false;
// 	for (int32 i = 0; i < Data.AdditionalActiveNodes.Num(); i++)
// 	{
// 		if (Node->FirstExecutionIndex <= Data.AdditionalActiveNodes[i] && Node->LastExecutionIndex >= Data.AdditionalActiveNodes[i])
// 		{
// 			bIsNodeActive = true;
// 			break;
// 		}
// 	}	
// 	
// 	Node->DebuggerUpdateCounter = DisplayedStepIndex;
// 	Node->bDebuggerMarkCurrentlyActive = bIsNodeActive && bIsShowingCurrentState;
// 	Node->bDebuggerMarkPreviouslyActive = bIsNodeActive && !bIsShowingCurrentState;
// 
// 	Node->bDebuggerMarkFlashActive = false;
// 	Node->bDebuggerMarkSearchTrigger = false;
// 	Node->bDebuggerMarkSearchFailedTrigger = false;
// 
// 	int32 SearchPathIdx = INDEX_NONE;
// 	int32 NumTriggers = 0;
// 	bool bTriggerOnly = false;
// 	for (int32 i = 0; i < Data.PathFromPrevious.Num(); i++)
// 	{
// 		const FConversationDebuggerInstance::FNodeFlowData& SearchStep = Data.PathFromPrevious[i];
// 		const bool bMatchesNodeIndex = (Node->FirstExecutionIndex <= SearchStep.ExecutionIndex && Node->LastExecutionIndex >= SearchStep.ExecutionIndex);
// 		if (SearchStep.bTrigger || SearchStep.bDiscardedTrigger)
// 		{
// 			NumTriggers++;
// 			if (bMatchesNodeIndex)
// 			{
// 				Node->bDebuggerMarkSearchTrigger = SearchStep.bTrigger;
// 				Node->bDebuggerMarkSearchFailedTrigger = SearchStep.bDiscardedTrigger;
// 				bTriggerOnly = true;
// 			}
// 		}
// 		else if (bMatchesNodeIndex)
// 		{
// 			SearchPathIdx = i;
// 			bTriggerOnly = false;
// 		}
// 	}
// 
// 	Node->bDebuggerMarkSearchSucceeded = (SearchPathIdx != INDEX_NONE) && Data.PathFromPrevious[SearchPathIdx].bPassed;
// 	Node->bDebuggerMarkSearchFailed = (SearchPathIdx != INDEX_NONE) && !Data.PathFromPrevious[SearchPathIdx].bPassed;
// 	Node->DebuggerSearchPathIndex = bTriggerOnly ? 0 : FMath::Max(-1, SearchPathIdx - NumTriggers);
// 	Node->DebuggerSearchPathSize = Data.PathFromPrevious.Num() - NumTriggers;
}

void FConversationDebugger::UpdateAssetFlags(const struct FConversationDebuggerInstance& Data, class UConversationGraphNode* Node, int32 StepIdx)
{
// 	if (Node == NULL)
// 	{
// 		return;
// 	}
// 
// 	// special case for marking root when out of nodes
// 	if (Node == RootNode.Get())
// 	{
// 		const bool bIsNodeActive = (Data.ActivePath.Num() == 0) && (StepIdx >= 0);
// 		const bool bIsShowingCurrentState = IsShowingCurrentState();
// 
// 		Node->bDebuggerMarkCurrentlyActive = bIsNodeActive && bIsShowingCurrentState;
// 		Node->bDebuggerMarkPreviouslyActive = bIsNodeActive && !bIsShowingCurrentState;
// 		DisplayedStepIndex = StepIdx;
// 	}
// 
// 	for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
// 	{
// 		UEdGraphPin* Pin = Node->Pins[PinIdx];
// 		if (Pin->Direction != EGPD_Output)
// 		{
// 			continue;
// 		}
// 
// 		for (int32 i = 0; i < Pin->LinkedTo.Num(); i++)
// 		{
// 			UConversationGraphNode* LinkedNode = Cast<UConversationGraphNode>(Pin->LinkedTo[i]->GetOwningNode());
// 			if (LinkedNode)
// 			{
// 				UBTNode* BTNode = Cast<UBTNode>(LinkedNode->NodeInstance);
// 				if (BTNode)
// 				{
// 					SetNodeFlags(Data, LinkedNode, BTNode);
// 					SetNodeRuntimeDescription(Data.RuntimeDesc, LinkedNode, BTNode);
// 				}
// 
// 				for (int32 iAux = 0; iAux < LinkedNode->Decorators.Num(); iAux++)
// 				{
// 					UConversationGraphNode_Decorator* DecoratorNode = Cast<UConversationGraphNode_Decorator>(LinkedNode->Decorators[iAux]);
// 					UBTAuxiliaryNode* AuxNode = DecoratorNode ? Cast<UBTAuxiliaryNode>(DecoratorNode->NodeInstance) : NULL;
// 					if (AuxNode)
// 					{
// 						SetNodeFlags(Data, DecoratorNode, AuxNode);
// 						SetNodeRuntimeDescription(Data.RuntimeDesc, DecoratorNode, AuxNode);
// 
// 						// pass restart trigger to parent graph node for drawing
// 						LinkedNode->bDebuggerMarkSearchTrigger |= DecoratorNode->bDebuggerMarkSearchTrigger;
// 						LinkedNode->bDebuggerMarkSearchFailedTrigger |= DecoratorNode->bDebuggerMarkSearchFailedTrigger;
// 					}
// 
// 					UConversationGraphNode_CompositeDecorator* CompDecoratorNode = Cast<UConversationGraphNode_CompositeDecorator>(LinkedNode->Decorators[iAux]);
// 					if (CompDecoratorNode)
// 					{
// 						SetCompositeDecoratorFlags(Data, CompDecoratorNode);
// 						SetCompositeDecoratorRuntimeDescription(Data.RuntimeDesc, CompDecoratorNode);
// 
// 						// pass restart trigger to parent graph node for drawing
// 						LinkedNode->bDebuggerMarkSearchTrigger |= CompDecoratorNode->bDebuggerMarkSearchTrigger;
// 						LinkedNode->bDebuggerMarkSearchFailedTrigger |= CompDecoratorNode->bDebuggerMarkSearchFailedTrigger;
// 					}
// 				}
// 
// 				for (int32 iAux = 0; iAux < LinkedNode->Services.Num(); iAux++)
// 				{
// 					UConversationGraphNode_Service* ServiceNode = Cast<UConversationGraphNode_Service>(LinkedNode->Services[iAux]);
// 					UBTAuxiliaryNode* AuxNode = ServiceNode ? Cast<UBTAuxiliaryNode>(ServiceNode->NodeInstance) : NULL;
// 					if (AuxNode)
// 					{
// 						SetNodeFlags(Data, ServiceNode, AuxNode);
// 						SetNodeRuntimeDescription(Data.RuntimeDesc, ServiceNode, AuxNode);
// 					}
// 				}
// 
// 				UpdateAssetFlags(Data, LinkedNode, StepIdx);
// 			}
// 		}
// 	}
}

void FConversationDebugger::UpdateAssetRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UConversationGraphNode* Node)
{
// 	if (Node == NULL)
// 	{
// 		return;
// 	}
// 
// 	for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
// 	{
// 		UEdGraphPin* Pin = Node->Pins[PinIdx];
// 		if (Pin->Direction != EGPD_Output)
// 		{
// 			continue;
// 		}
// 
// 		for (int32 i = 0; i < Pin->LinkedTo.Num(); i++)
// 		{
// 			UConversationGraphNode* LinkedNode = Cast<UConversationGraphNode>(Pin->LinkedTo[i]->GetOwningNode());
// 			if (LinkedNode)
// 			{
// 				UBTNode* BTNode = Cast<UBTNode>(LinkedNode->NodeInstance);
// 				if (BTNode)
// 				{
// 					SetNodeRuntimeDescription(RuntimeDescriptions, LinkedNode, BTNode);
// 				}
// 
// 				for (int32 iAux = 0; iAux < LinkedNode->Decorators.Num(); iAux++)
// 				{
// 					UConversationGraphNode_Decorator* DecoratorNode = Cast<UConversationGraphNode_Decorator>(LinkedNode->Decorators[iAux]);
// 					UBTAuxiliaryNode* AuxNode = DecoratorNode ? Cast<UBTAuxiliaryNode>(DecoratorNode->NodeInstance) : NULL;
// 					if (AuxNode)
// 					{
// 						SetNodeRuntimeDescription(RuntimeDescriptions, DecoratorNode, AuxNode);
// 					}
// 
// 					UConversationGraphNode_CompositeDecorator* CompDecoratorNode = Cast<UConversationGraphNode_CompositeDecorator>(LinkedNode->Decorators[iAux]);
// 					if (CompDecoratorNode)
// 					{
// 						SetCompositeDecoratorRuntimeDescription(RuntimeDescriptions, CompDecoratorNode);
// 					}
// 				}
// 
// 				for (int32 iAux = 0; iAux < LinkedNode->Services.Num(); iAux++)
// 				{
// 					UConversationGraphNode_Service* ServiceNode = Cast<UConversationGraphNode_Service>(LinkedNode->Services[iAux]);
// 					UBTAuxiliaryNode* AuxNode = ServiceNode ? Cast<UBTAuxiliaryNode>(ServiceNode->NodeInstance) : NULL;
// 					if (AuxNode)
// 					{
// 						SetNodeRuntimeDescription(RuntimeDescriptions, ServiceNode, AuxNode);
// 					}
// 				}
// 
// 				UpdateAssetRuntimeDescription(RuntimeDescriptions, LinkedNode);
// 			}
// 		}
// 	}
}

// void FConversationDebugger::SetNodeRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UConversationGraphNode* Node, class UBTNode* NodeInstance)
// {
// 	Node->DebuggerRuntimeDescription = RuntimeDescriptions.IsValidIndex(NodeInstance->GetExecutionIndex()) ?
// 		RuntimeDescriptions[NodeInstance->GetExecutionIndex()] : FString();
// }

// void FConversationDebugger::SetCompositeDecoratorRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UConversationGraphNode_CompositeDecorator* Node)
// {
// 	Node->DebuggerRuntimeDescription.Empty();
// 	for (int32 i = Node->FirstExecutionIndex; i <= Node->LastExecutionIndex; i++)
// 	{
// 		if (RuntimeDescriptions.IsValidIndex(i) && RuntimeDescriptions[i].Len())
// 		{
// 			if (Node->DebuggerRuntimeDescription.Len())
// 			{
// 				Node->DebuggerRuntimeDescription.AppendChar(TEXT('\n'));
// 			}
// 
// 			Node->DebuggerRuntimeDescription += FString::Printf(TEXT("[%d] %s"), i, *RuntimeDescriptions[i].Replace(TEXT("\n"), TEXT(", ")));
// 		}
// 	}
// }

void FConversationDebugger::CollectBreakpointsFromAsset(class UConversationGraphNode* Node)
{
// 	if (Node == NULL)
// 	{
// 		return;
// 	}
// 
// 	for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
// 	{
// 		UEdGraphPin* Pin = Node->Pins[PinIdx];
// 		if (Pin->Direction != EGPD_Output)
// 		{
// 			continue;
// 		}
// 
// 		for (int32 i = 0; i < Pin->LinkedTo.Num(); i++)
// 		{
// 			UConversationGraphNode* LinkedNode = Cast<UConversationGraphNode>(Pin->LinkedTo[i]->GetOwningNode());
// 			if (LinkedNode)
// 			{
// 				UBTNode* BTNode = Cast<UBTNode>(LinkedNode->NodeInstance);
// 				if (BTNode && LinkedNode->bHasBreakpoint && LinkedNode->bIsBreakpointEnabled)
// 				{
// 					ActiveBreakpoints.Add(BTNode->GetExecutionIndex());
// 				}
// 
// 				CollectBreakpointsFromAsset(LinkedNode);
// 			}
// 		}
// 	}
}

// int32 FConversationDebugger::FindMatchingDebuggerStack(UConversationComponent& TestInstance) const
// {
// #if USE_CONVERSATION_DEBUGGER
// 	if (TestInstance.DebuggerSteps.Num())
// 	{
// 		const FConversationExecutionStep& StepInfo = TestInstance.DebuggerSteps.Last();
// 		for (int32 i = 0; i < StepInfo.InstanceStack.Num(); i++)
// 		{
// 			if (StepInfo.InstanceStack[i].TreeAsset == TreeAsset)
// 			{
// 				return i;
// 			}
// 		}
// 	}
// #endif
// 
// 	return INDEX_NONE;
// }
// 
// UConversationComponent* FConversationDebugger::FindInstanceInActor(AActor* TestActor)
// {
// 	UConversationComponent* FoundInstance = NULL;
// 	if (TestActor)
// 	{
// 		APawn* TestPawn = Cast<APawn>(TestActor);
// 		if (TestPawn && TestPawn->GetController())
// 		{
// 			FoundInstance = TestPawn->GetController()->FindComponentByClass<UConversationComponent>();
// 		}
// 
// 		if (FoundInstance == NULL)
// 		{
// 			FoundInstance = TestActor->FindComponentByClass<UConversationComponent>();
// 		}
// 	}
// 
// 	return FoundInstance;
// }

void FConversationDebugger::FindLockedDebugActor(UWorld* World)
{
#if 0
	APlayerController* LocalPC = GEngine->GetFirstLocalPlayerController(World);
	if (LocalPC && LocalPC->GetHUD() && LocalPC->GetPawnOrSpectator())
	{
		APawn* SelectedPawn = NULL;
#if WITH_ENGINE
		const UEditorEngine* EEngine = Cast<UEditorEngine>(GEngine);
		for (FSelectionIterator It = EEngine->GetSelectedActorIterator(); It; ++It)
		{
			SelectedPawn = Cast<APawn>(*It);
			if (SelectedPawn)
			{
				break;
			}
		}
#endif //WITH_ENGINE

		UConversationComponent* TestInstance = FindInstanceInActor((APawn*)SelectedPawn);
		if (TestInstance)
		{
			TreeInstance = TestInstance;
#if USE_CONVERSATION_DEBUGGER
			ActiveStepIndex = TestInstance->DebuggerSteps.Num() - 1;
#endif
		}
	}
#endif
}

void FConversationDebugger::FindMatchingTreeInstance()
{
#if 0
	KnownInstances.Reset();

	// Find the world for the dedicated server if any, otherwise fallback to the PIE world
	UWorld* PlayWorld = nullptr;
	for (const FWorldContext& PieContext : GEditor->GetWorldContexts())
	{
		if (PieContext.WorldType == EWorldType::PIE && PieContext.World() != nullptr)
		{
			if (PieContext.RunAsDedicated)
			{
				PlayWorld = PieContext.World();
				break;
			}
			else if(!PlayWorld)
			{
				PlayWorld = PieContext.World();
				// Need to continue to see if their is a dedicated server.
			}
		}
	}
	 
	if (PlayWorld == NULL)
	{
		return;
	}

	UConversationComponent* MatchingComp = NULL;
	for (FActorIterator It(PlayWorld); It; ++It)
	{
		AActor* TestActor = *It;
		UConversationComponent* TestComp = TestActor ? TestActor->FindComponentByClass<UConversationComponent>() : nullptr;

		if (TestComp)
		{
			KnownInstances.Add(TestComp);

			const int32 MatchingIdx = FindMatchingDebuggerStack(*TestComp);
			if (MatchingIdx != INDEX_NONE)
			{
				MatchingComp = TestComp;

				if (TestActor->IsSelected())
				{
					TreeInstance = TestComp;
					return;
				}
			}
		}
	}

	if (MatchingComp != TreeInstance)
	{
		TreeInstance = MatchingComp;
		UpdateDebuggerViewOnInstanceChange();
	}
#endif
}

bool FConversationDebugger::IsDebuggerReady() const
{
	return bIsPIEActive;
}

bool FConversationDebugger::IsDebuggerRunning() const
{
//	return TreeInstance.IsValid() && (ActiveStepIndex != INDEX_NONE);
	return false;
}

bool FConversationDebugger::IsShowingCurrentState() const
{
#if USE_CONVERSATION_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.Num())
	{
		return (TreeInstance->DebuggerSteps.Num() - 1) == ActiveStepIndex;
	}
#endif

	return false;
}

int32 FConversationDebugger::GetShownStateIndex() const
{
#if USE_CONVERSATION_DEBUGGER
	if (TreeInstance.IsValid())
	{
		return (TreeInstance->DebuggerSteps.Num() - 1) - ActiveStepIndex;
	}
#endif

	return 0;
}

void FConversationDebugger::StepForwardInto()
{
#if USE_CONVERSATION_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepForwardIntoIdx);
#endif
}

static void ForEachGameWorld(const TFunction<void(UWorld*)>& Func)
{
	for (const FWorldContext& PieContext : GUnrealEd->GetWorldContexts())
	{
		UWorld* PlayWorld = PieContext.World();
		if (PlayWorld && PlayWorld->IsGameWorld())
		{
			Func(PlayWorld);
		}
	}
}

static bool AreAllGameWorldPaused()
{
	bool bPaused = true;
	ForEachGameWorld([&](UWorld* World)
	{ 
		bPaused = bPaused && World->bDebugPauseExecution; 
	});
	return bPaused;
}

bool FConversationDebugger::CanStepForwardInto() const
{
	return AreAllGameWorldPaused() && (StepForwardIntoIdx != INDEX_NONE);
}

void FConversationDebugger::StepForwardOver()
{
#if USE_CONVERSATION_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepForwardOverIdx);
#endif
}

bool FConversationDebugger::CanStepForwardOver() const
{
	return AreAllGameWorldPaused() && (StepForwardOverIdx != INDEX_NONE);
}

void FConversationDebugger::StepOut()
{
#if USE_CONVERSATION_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepOutIdx);
#endif
}

bool FConversationDebugger::CanStepOut() const
{
	return AreAllGameWorldPaused() && (StepOutIdx != INDEX_NONE);
}

void FConversationDebugger::StepBackInto()
{
#if USE_CONVERSATION_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepBackIntoIdx);
#endif
}

bool FConversationDebugger::CanStepBackInto() const
{
	return AreAllGameWorldPaused() && (StepBackIntoIdx != INDEX_NONE);
}

void FConversationDebugger::StepBackOver()
{
#if USE_CONVERSATION_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepBackOverIdx);
#endif
}

bool FConversationDebugger::CanStepBackOver() const
{
	return AreAllGameWorldPaused() && (StepBackOverIdx != INDEX_NONE);
}

void FConversationDebugger::UpdateCurrentStep(int32 PrevStepIdx, int32 NewStepIdx)
{
#if USE_CONVERSATION_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(NewStepIdx))
	{
		const int32 CurInstaceIdx = FindActiveInstanceIdx(PrevStepIdx);
		const int32 NewInstanceIdx = FindActiveInstanceIdx(NewStepIdx);

		const FConversationExecutionStep& CurStepInfo = TreeInstance->DebuggerSteps[PrevStepIdx];
		const FConversationExecutionStep& NewStepInfo = TreeInstance->DebuggerSteps[NewStepIdx];

		ActiveStepIndex = NewStepIdx;

		if (NewInstanceIdx != INDEX_NONE && NewStepInfo.InstanceStack[NewInstanceIdx].TreeAsset != TreeAsset)
		{
			if (CurInstaceIdx != NewInstanceIdx || 
				CurStepInfo.InstanceStack[CurInstaceIdx].TreeAsset != NewStepInfo.InstanceStack[NewInstanceIdx].TreeAsset)
			{
				if(EditorOwner.IsValid())
				{
					EditorOwner.Pin()->DebuggerSwitchAsset(NewStepInfo.InstanceStack[NewInstanceIdx].TreeAsset);
				}
				UpdateCurrentSubtree();
			}
		}

		if (NewStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex))
		{
			const FConversationDebuggerInstance& ShowInstance = NewStepInfo.InstanceStack[DebuggerInstanceIndex];
			UpdateAssetFlags(ShowInstance, RootNode.Get(), ActiveStepIndex);
		}
		else
		{
			ActiveStepIndex = INDEX_NONE;

			FConversationDebuggerInstance EmptyData;
			UpdateAssetFlags(EmptyData, RootNode.Get(), INDEX_NONE);
		}

		UpdateDebuggerViewOnStepChange();
		UpdateAvailableActions();
	}
#endif
}

bool FConversationDebugger::HasContinuousNextStep() const
{
#if USE_CONVERSATION_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex + 1))
	{
		const FConversationExecutionStep& NextStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex + 1];
		const FConversationExecutionStep& CurStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];

		if (CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) &&
			CurStepInfo.InstanceStack.Num() == NextStepInfo.InstanceStack.Num() &&
			CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset == NextStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
		{
			return true;
		}
	}
#endif
	return false;
}

bool FConversationDebugger::HasContinuousPrevStep() const
{
#if USE_CONVERSATION_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex - 1))
	{
		const FConversationExecutionStep& PrevStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex - 1];
		const FConversationExecutionStep& CurStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];

		if (CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) &&
			CurStepInfo.InstanceStack.Num() == PrevStepInfo.InstanceStack.Num() &&
			CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset == PrevStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
		{
			return true;
		}
	}
#endif
	return false;
}

void FConversationDebugger::OnActiveNodeChanged(const TArray<uint16>& ActivePath, const TArray<uint16>& PrevStepPath)
{
	bool bShouldPause = false;
	StoppedOnBreakpointExecutionIndex = MAX_uint16;
	
	// breakpoints: check only nodes, that have changed from previous state
	// (e.g. breakpoint on sequence, it would break multiple times for every child
	// but we want only once: when it becomes active)

	for (int32 i = 0; i < ActivePath.Num(); i++)
	{
		const uint16 TestExecutionIndex = ActivePath[i];
		if (!PrevStepPath.Contains(TestExecutionIndex))
		{
			if (ActiveBreakpoints.Contains(TestExecutionIndex))
			{
				bShouldPause = true;
				StoppedOnBreakpointExecutionIndex = TestExecutionIndex;
				break;
			}
		}
	}

	if (bShouldPause)
	{
		if (EditorOwner.IsValid())
		{
			EditorOwner.Pin()->FocusWindow(TreeAsset);
		}

		PausePlaySession();
	}
}

void FConversationDebugger::StopPlaySession()
{
	if (GUnrealEd->PlayWorld)
	{
		GEditor->RequestEndPlayMap();
 
		// @TODO: we need a unified flow to leave debugging mode from the different debuggers to prevent strong coupling between modules.
		// Each debugger (Blueprint & BehaviorTree for now) could then take the appropriate actions to resume the session.
		if (FSlateApplication::Get().InKismetDebuggingMode())
		{
			FSlateApplication::Get().LeaveDebuggingMode();
		}
	}
}

void FConversationDebugger::PausePlaySession()
{
	if (GUnrealEd->SetPIEWorldsPaused(true))
	{
		GUnrealEd->PlaySessionPaused();
	}
}

void FConversationDebugger::ResumePlaySession()
{
	if (GUnrealEd->SetPIEWorldsPaused(false))
	{
		// @TODO: we need a unified flow to leave debugging mode from the different debuggers to prevent strong coupling between modules.
		// Each debugger (Blueprint & BehaviorTree for now) could then take the appropriate actions to resume the session.
		if (FSlateApplication::Get().InKismetDebuggingMode())
		{
			FSlateApplication::Get().LeaveDebuggingMode();
		}

		GUnrealEd->PlaySessionResumed();
	}
}

bool FConversationDebugger::IsPlaySessionPaused()
{
	return AreAllGameWorldPaused();
}

bool FConversationDebugger::IsPlaySessionRunning()
{
	return !AreAllGameWorldPaused();
}

bool FConversationDebugger::IsPIESimulating()
{
	return GEditor->bIsSimulatingInEditor || GEditor->PlayWorld;
}

bool FConversationDebugger::IsPIENotSimulating()
{
	return !GEditor->bIsSimulatingInEditor && (GEditor->PlayWorld == NULL);
}

void FConversationDebugger::OnBreakpointAdded(class UConversationGraphNode* Node)
{
// 	if (IsDebuggerReady())
// 	{
// 		UBTNode* BTNode = Cast<UBTNode>(Node->NodeInstance);
// 		if (BTNode)
// 		{
// 			ActiveBreakpoints.AddUnique(BTNode->GetExecutionIndex());
// 		}
// 	}
}

void FConversationDebugger::OnBreakpointRemoved(class UConversationGraphNode* Node)
{
// 	if (IsDebuggerReady())
// 	{
// 		UBTNode* BTNode = Cast<UBTNode>(Node->NodeInstance);
// 		if (BTNode)
// 		{
// 			ActiveBreakpoints.RemoveSingleSwap(BTNode->GetExecutionIndex());
// 		}
// 	}
}

void FConversationDebugger::UpdateDebuggerViewOnInstanceChange()
{
#if USE_CONVERSATION_DEBUGGER
	UBlackboardData* BBAsset = EditorOwner.IsValid() ? EditorOwner.Pin()->GetBlackboardData() : nullptr;
	if (TreeInstance.IsValid() &&
		TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex) &&
		TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack.IsValidIndex(DebuggerInstanceIndex))
	{
		const FConversationDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];
		if (ShowInstance.TreeAsset)
		{
			BBAsset = ShowInstance.TreeAsset->BlackboardAsset;
		}
	}

	OnDebuggedBlackboardChangedEvent.Broadcast(BBAsset);

	if (DebuggerInstanceIndex != INDEX_NONE)
	{
		Refresh();
	}
	else
	{
		ClearDebuggerState(/*bKeepSubtreeData=*/true);
	}
#endif
}

void FConversationDebugger::UpdateDebuggerViewOnStepChange()
{
#if USE_CONVERSATION_DEBUGGER
	if (IsDebuggerRunning() &&
		TreeInstance.IsValid() &&
		TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		const FConversationExecutionStep& ShowStep = TreeInstance->DebuggerSteps[ActiveStepIndex];

		SavedTimestamp = ShowStep.TimeStamp;
		SavedValues = ShowStep.BlackboardValues;
	}
#endif
}

void FConversationDebugger::UpdateDebuggerViewOnTick()
{
#if USE_CONVERSATION_DEBUGGER
	if (IsDebuggerRunning() && TreeInstance.IsValid())
	{
		const float GameTime = GEditor && GEditor->PlayWorld ? GEditor->PlayWorld->GetTimeSeconds() : 0.0f;
		CurrentTimestamp = GameTime;

		TreeInstance->StoreDebuggerBlackboard(CurrentValues);
	}
#endif
}

FText FConversationDebugger::FindValueForKey(const FName& InKeyName, bool bUseCurrentState) const
{
#if USE_CONVERSATION_DEBUGGER
	if (IsDebuggerRunning() &&
		TreeInstance.IsValid())
	{
		const TMap<FName, FString>* MapToQuery = nullptr;
		if(bUseCurrentState)
		{
			MapToQuery = &CurrentValues;
		}
		else if(TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
		{
			MapToQuery = &TreeInstance->DebuggerSteps[ActiveStepIndex].BlackboardValues;
		}

		if(MapToQuery != nullptr)
		{
			const FString* FindValue = MapToQuery->Find(InKeyName);
			if(FindValue != nullptr)
			{
				return FText::FromString(*FindValue);
			}	
		}
	}

	return FText();
#endif
	return FText();
}

float FConversationDebugger::GetTimeStamp(bool bUseCurrentState) const
{
	return bUseCurrentState ? CurrentTimestamp : SavedTimestamp;
}

FString FConversationDebugger::GetDebuggedInstanceDesc() const
{
// 	UConversationComponent* BTComponent = TreeInstance.Get();
// 	return BTComponent ? DescribeInstance(*BTComponent) : 
	return NSLOCTEXT("BlueprintEditor", "DebugActorNothingSelected", "No debug object selected").ToString();
}

// FString FConversationDebugger::DescribeInstance(UConversationComponent& InstanceToDescribe) const
// {
// 	FString ActorDesc;
// 	if (InstanceToDescribe.GetOwner())
// 	{
// 		AController* TestController = Cast<AController>(InstanceToDescribe.GetOwner());
// 		ActorDesc = TestController ?
// 			TestController->GetName() :
// 			InstanceToDescribe.GetOwner()->GetActorLabel();
// 	}
// 
// 	return ActorDesc;
// }

// void FConversationDebugger::OnInstanceSelectedInDropdown(UConversationComponent* SelectedInstance)
// {
// 	if (SelectedInstance)
// 	{
// 		ClearDebuggerState();
// 
// 		AController* OldController = TreeInstance.IsValid() ? Cast<AController>(TreeInstance->GetOwner()) : NULL;
// 		APawn* OldPawn = OldController != NULL ? OldController->GetPawn() : NULL;
// 		USelection* SelectedActors = GEditor ? GEditor->GetSelectedActors() : NULL;
// 		if (SelectedActors)
// 		{
// 			SelectedActors->DeselectAll();
// 		}
// 
// 		TreeInstance = SelectedInstance;
// 
// 		if (SelectedActors && SelectedInstance && SelectedInstance->GetOwner())
// 		{
// 			AController* TestController = Cast<AController>(SelectedInstance->GetOwner());
// 			APawn* Pawn = TestController != NULL ? TestController->GetPawn() : NULL;
// 			if (Pawn)
// 			{
// 				SelectedActors->Select(Pawn);
// 			}
// 		}
// 
// 		Refresh();
// 	}
// }
// 
// void FConversationDebugger::GetMatchingInstances(TArray<UConversationComponent*>& MatchingInstances)
// {
// 	for (int32 i = KnownInstances.Num() - 1; i >= 0; i--)
// 	{
// 		UConversationComponent* TestInstance = KnownInstances[i].Get();
// 		if (TestInstance == NULL)
// 		{
// 			KnownInstances.RemoveAt(i);
// 			continue;
// 		}
// 
// 		const int32 StackIdx = FindMatchingDebuggerStack(*TestInstance);
// 		if (StackIdx != INDEX_NONE)
// 		{
// 			MatchingInstances.Add(TestInstance);
// 		}
// 	}
// }

void FConversationDebugger::InitializeFromParent(class FConversationDebugger* ParentDebugger)
{
	ClearDebuggerState();

#if USE_CONVERSATION_DEBUGGER
	TreeInstance = ParentDebugger->TreeInstance;
	ActiveStepIndex = ParentDebugger->ActiveStepIndex;

	UpdateDebuggerInstance();
	UpdateAvailableActions();

	if (TreeInstance.IsValid() &&
		TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex) &&
		TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack.IsValidIndex(DebuggerInstanceIndex))
	{
		const FConversationDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];
		UpdateAssetFlags(ShowInstance, RootNode.Get(), ActiveStepIndex);
	}
#endif
}

int32 FConversationDebugger::FindActiveInstanceIdx(int32 StepIdx) const
{
#if USE_CONVERSATION_DEBUGGER
	const FConversationExecutionStep& StepInfo = TreeInstance->DebuggerSteps[StepIdx];
	for (int32 i = StepInfo.InstanceStack.Num() - 1; i >= 0; i--)
	{
		if (StepInfo.InstanceStack[i].IsValid())
		{
			return i;
		}
	}
#endif

	return INDEX_NONE;
}

void FConversationDebugger::UpdateCurrentSubtree()
{
	bIsCurrentSubtree = false;

#if USE_CONVERSATION_DEBUGGER
	if (TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		const FConversationExecutionStep& StepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];

		// assume that top instance is always valid, so it won't take away step buttons when tree is finished as out of nodes
		// current subtree = no child instances, or child instances are not valid
		bIsCurrentSubtree = ((DebuggerInstanceIndex == 0) || (StepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) && StepInfo.InstanceStack[DebuggerInstanceIndex].IsValid())) &&
			(!StepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex + 1) || !StepInfo.InstanceStack[DebuggerInstanceIndex + 1].IsValid());
	}
#endif
}

// static int32 GetNumActiveInstances(const FConversationExecutionStep& StepInfo, class UConversationDatabase*& ActiveSubtree)
// {
// 	for (int32 Idx = StepInfo.InstanceStack.Num() - 1; Idx >= 0; Idx--)
// 	{
// 		//if (StepInfo.InstanceStack[Idx].ActivePath.Num())
// 		{
// 			ActiveSubtree = StepInfo.InstanceStack[Idx].TreeAsset;
// 			return Idx + 1;
// 		}
// 	}
// 
// 	ActiveSubtree = NULL;
// 	return 0;
// }

void FConversationDebugger::UpdateAvailableActions()
{
	StepForwardIntoIdx = INDEX_NONE;
	StepForwardOverIdx = INDEX_NONE;
	StepBackIntoIdx = INDEX_NONE;
	StepBackOverIdx = INDEX_NONE;
	StepOutIdx = INDEX_NONE;

#if USE_CONVERSATION_DEBUGGER
	UConversationComponent* TreeInstancePtr = TreeInstance.Get();
	if (TreeInstancePtr && TreeInstancePtr->DebuggerSteps.IsValidIndex(ActiveStepIndex) && DebuggerInstanceIndex >= 0)
	{
		const FConversationExecutionStep& CurStepInfo = TreeInstancePtr->DebuggerSteps[ActiveStepIndex];

		if (TreeInstancePtr->DebuggerSteps.IsValidIndex(ActiveStepIndex - 1))
		{
			StepBackIntoIdx = ActiveStepIndex - 1;
		}

		if (TreeInstancePtr->DebuggerSteps.IsValidIndex(ActiveStepIndex + 1))
		{
			StepForwardIntoIdx = ActiveStepIndex + 1;
		}

		UConversationDatabase* CurTree = CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) ?
			CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset : NULL;
		const int32 CurStepInstances = DebuggerInstanceIndex + 1;

		for (int32 TestStepIndex = ActiveStepIndex - 1; TestStepIndex >= 0; TestStepIndex--)
		{
			const FConversationExecutionStep& TestStepInfo = TreeInstancePtr->DebuggerSteps[TestStepIndex];
			UConversationDatabase* TestTree = NULL;
			const int32 TestStepInstances = GetNumActiveInstances(TestStepInfo, TestTree);

			StepBackOverIdx = TestStepIndex;

			// keep going only if the execution is moving to a sub-tree
			if (TestStepInstances <= CurStepInstances ||
				TestStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset != CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
			{
				break;
			}
		}

		for (int32 TestStepIndex = ActiveStepIndex + 1; TestStepIndex < TreeInstancePtr->DebuggerSteps.Num(); TestStepIndex++)
		{
			const FConversationExecutionStep& TestStepInfo = TreeInstancePtr->DebuggerSteps[TestStepIndex];
			UConversationDatabase* TestTree = NULL;
			int32 TestStepInstances = GetNumActiveInstances(TestStepInfo, TestTree);

			StepForwardOverIdx = TestStepIndex;

			// keep going only if the execution is moving to a sub-tree
			if (TestStepInstances <= CurStepInstances ||
				TestStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset != CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
			{
				break;
			}
		}

		if (CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) && CurStepInfo.InstanceStack[DebuggerInstanceIndex].ActivePath.Num())
		{
			for (int32 TestStepIndex = ActiveStepIndex + 1; TestStepIndex < TreeInstancePtr->DebuggerSteps.Num(); TestStepIndex++)
			{
				const FConversationExecutionStep& TestStepInfo = TreeInstancePtr->DebuggerSteps[TestStepIndex];
				UConversationDatabase* TestTree = NULL;
				int32 TestStepInstances = GetNumActiveInstances(TestStepInfo, TestTree);

				if (TestStepInstances < CurStepInstances ||
					TestStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset != CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
				{
					// execution left current subtree
					StepOutIdx = TestStepIndex;
					break;
				}
			}
		}
	}
#endif
}
