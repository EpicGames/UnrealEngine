// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Stats/Stats.h"
#include "Tickable.h"

enum class EStatFlags : uint8;
struct FStatGroup_STATGROUP_Tickables;
struct TStatIdData;

class AActor;
class APawn;
class FConversationEditor;
class UConversationDatabase;
class UBlackboardData;
class UConversationGraphNode;

class FConversationDebugger : public FTickableGameObject
{
public:
	FConversationDebugger();
	~FConversationDebugger();

	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FConversationEditorTickHelper, STATGROUP_Tickables); }

	/** Refresh the debugging information we are displaying (only when paused, as Tick() updates when running) */
	void Refresh();

	void Setup(UConversationDatabase* InTreeAsset, TSharedRef<class FConversationEditor> InEditorOwner);

	/** Store the root node for easy access if we have not already */
	void CacheRootNode();

	bool IsDebuggerReady() const;
	bool IsDebuggerRunning() const;
	bool IsShowingCurrentState() const;
	int32 GetShownStateIndex() const;

	void OnObjectSelected(UObject* Object);
	void OnAIDebugSelected(const APawn* Pawn);
	//void OnTreeStarted(const UConversationComponent& OwnerComp, const UConversationDatabase& InTreeAsset);
	void OnBeginPIE(const bool bIsSimulating);
	void OnEndPIE(const bool bIsSimulating);
	void OnPausePIE(const bool bIsSimulating);

	void OnBreakpointAdded(UConversationGraphNode* Node);
	void OnBreakpointRemoved(UConversationGraphNode* Node);

	void StepBackInto();
	void StepBackOver();
	void StepForwardInto();
	void StepForwardOver();
	void StepOut();
	bool CanStepBackInto() const;
	bool CanStepBackOver() const;
	bool CanStepForwardInto() const;
	bool CanStepForwardOver() const;
	bool CanStepOut() const;

	static void StopPlaySession();
	static void PausePlaySession();
	static void ResumePlaySession();
	static bool IsPlaySessionPaused();
	static bool IsPlaySessionRunning();
	static bool IsPIESimulating();
	static bool IsPIENotSimulating();

	FString GetDebuggedInstanceDesc() const;
	//FString DescribeInstance(UConversationComponent& InstanceToDescribe) const;
	//void OnInstanceSelectedInDropdown(UConversationComponent* SelectedInstance);
	//void GetMatchingInstances(TArray<UConversationComponent*>& MatchingInstances);

	void InitializeFromParent(class FConversationDebugger* ParentDebugger);
	bool HasContinuousNextStep() const;
	bool HasContinuousPrevStep() const;

	/**
	 * Find a (display) value for a given key.
	 * @param	InKeyName			Key to find a value for
	 * @param	bUseCurrentState	Whether to use the current (present) state or the state at the active step index
	 * @return the value to display to the user.
	 */
	FText FindValueForKey(const FName& InKeyName, bool bUseCurrentState) const;

	/** Gets the timestamp to be displayed, either current or saved */
	float GetTimeStamp(bool bUseCurrentState) const;

	/** Delegate fired when the debugged blackboard is changed */
	DECLARE_EVENT_OneParam(FConversationDebugger, FOnDebuggedBlackboardChanged, UBlackboardData*);
	FOnDebuggedBlackboardChanged& OnDebuggedBlackboardChanged() { return OnDebuggedBlackboardChangedEvent; }

private:

	/** owning editor */
	TWeakPtr<FConversationEditor> EditorOwner;

	/** asset for debugging */
	UConversationDatabase* TreeAsset;
// 
// 	/** root node in asset's graph */
// 	TWeakObjectPtr<class UConversationGraphNode_Root> RootNode;

	/** instance for debugging */
//	TWeakObjectPtr<class UConversationComponent> TreeInstance;

	/** matching debugger instance index from component's stack */
	int32 DebuggerInstanceIndex;

	/** index of state from buffer to show */
	int32 ActiveStepIndex;

	/** index of displayed step, used to detect changes */
	int32 DisplayedStepIndex;

	/** id of last valid step */
	int32 LastValidStepId;

	/** indices of display steps for different step actions */
	int32 StepForwardIntoIdx;
	int32 StepForwardOverIdx;
	int32 StepBackIntoIdx;
	int32 StepBackOverIdx;
	int32 StepOutIdx;

	/** execution indices of currently active breakpoints */
	TArray<uint16> ActiveBreakpoints;

	/** all known BT instances, cached for dropdown list */
//	TArray<TWeakObjectPtr<class UConversationComponent> > KnownInstances;

	/** cached PIE state */
	uint32 bIsPIEActive : 1;

	/** set when debugger instance is currently active one */
	uint32 bIsCurrentSubtree : 1;

	/** execution index of node that caused activated the breakpoint */
	uint16 StoppedOnBreakpointExecutionIndex;

	/** Lookup of currently debugged blackboard values */
	TMap<FName, FString> SavedValues;
	TMap<FName, FString> CurrentValues;

	/** Debugger timestamps */
	float SavedTimestamp;
	float CurrentTimestamp;

	/** set value of DebuggerInstanceIndex variable */
	void UpdateDebuggerInstance();

	/** clear all runtime variables */
	void ClearDebuggerState(bool bKeepSubtree = false);

	/** try using breakpoints on node change */
	void OnActiveNodeChanged(const TArray<uint16>& ActivePath, const TArray<uint16>& PrevStepPath);

	/** scan all actors and try to find matching BT component
	  * used only when user starts PIE before opening editor */
	void FindMatchingTreeInstance();

	/** find index on execution instance stack of matching tree asset */
//	int32 FindMatchingDebuggerStack(UConversationComponent& TestInstance) const;

	/** find BT component in given actor */
//	UConversationComponent* FindInstanceInActor(AActor* TestActor);

	/** try to find pawn currently locked by ai debug tool */
	void FindLockedDebugActor(class UWorld* World);

	/** recursively collect all breakpoint indices from child nodes */
	void CollectBreakpointsFromAsset(class UConversationGraphNode* Node);

	/** recursively update node flags on all child nodes */
	void UpdateAssetFlags(const struct FConversationDebuggerInstance& Data, class UConversationGraphNode* Node, int32 StepIdx);

	/** set debugger flags on GraphNode */
//	void SetNodeFlags(const struct FConversationDebuggerInstance& Data, class UConversationGraphNode* Node, class UBTNode* NodeInstance);

	/** set debugger flags on GraphNode for composite decorator */
	void SetCompositeDecoratorFlags(const struct FConversationDebuggerInstance& Data, class UConversationGraphNode_CompositeDecorator* Node);

	/** recursively update node flags on all child nodes */
	void UpdateAssetRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UConversationGraphNode* Node);

	/** set debugger flags on GraphNode */
//	void SetNodeRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UConversationGraphNode* Node, class UBTNode* NodeInstance);

	/** set debugger flags on GraphNode for composite decorator */
//	void SetCompositeDecoratorRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UConversationGraphNode_CompositeDecorator* Node);

	/** updates variables in debugger details view */
	void UpdateDebuggerViewOnInstanceChange();
	void UpdateDebuggerViewOnStepChange();
	void UpdateDebuggerViewOnTick();

	/** find valid instance for given debugger step */
	int32 FindActiveInstanceIdx(int32 StepIdx) const;

	/** check if currently debugged instance is active subtree */
	void UpdateCurrentSubtree();

	/** updates currently displayed execution step */
	void UpdateCurrentStep(int32 PrevStepIdx, int32 NewStepIdx);

	/** updates button states */
	void UpdateAvailableActions();

	/** Delegate fired when the debugged blackboard is changed */
	FOnDebuggedBlackboardChanged OnDebuggedBlackboardChangedEvent;
};
