// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditor.h"
#include "DetailsViewArgs.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Misc/PackageName.h"
#include "UnrealEdGlobals.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "ConversationGraph.h"
#include "ConversationGraphNode.h"
#include "ConversationDatabase.h"

#include "Misc/Paths.h"
#include "PropertyEditorModule.h"

#include "ConversationDebugger.h"
#include "FindInConversationGraph.h"
#include "IDetailsView.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"

#include "ConversationEditorModes.h"
#include "ConversationEditorToolbar.h"
#include "ConversationEditorTabFactories.h"
#include "ConversationEditorCommands.h"
#include "ConversationEditorTabs.h"
#include "ConversationEditorUtils.h"
#include "ConversationCompiler.h"

#include "ClassViewerFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "SConversationTreeEditor.h"

//////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "ConversationEditor"

const FName FConversationEditor::GraphViewMode("GraphViewMode");
const FName FConversationEditor::TreeViewMode("TreeViewMode");

const FName ConversationEditorAppIdentifier("ConversationEditorApp");

//////////////////////////////////////////////////////////////////////////

//@TODO: CONVERSATION: DEBUGGER
bool HasBreakpoint(UConversationGraphNode* Node) { return false; }
bool HasEnabledBreakpoint(UConversationGraphNode* Node) { return false; }
void SetBreakpointEnabled(UConversationGraphNode* Node, bool bNewState) {}
void RemoveBreakpoint(UConversationGraphNode* Node) {}

//////////////////////////////////////////////////////////////////////////

class FNewNodeClassFilter : public IClassViewerFilter
{
public:
	FNewNodeClassFilter(UClass* InBaseClass)
		: BaseClass(InBaseClass)
	{
	}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		if (InClass != nullptr)
		{
			return InClass->IsChildOf(BaseClass);
		}
		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InUnloadedClassData->IsChildOf(BaseClass);
	}

private:
	UClass* BaseClass;
};


//////////////////////////////////////////////////////////////////////////

FConversationEditor::FConversationEditor()
{
	ConversationAsset = nullptr;

	bCheckDirtyOnAssetSave = true;
}

FConversationEditor::~FConversationEditor()
{
	Debugger.Reset();
}

TSharedRef<FConversationEditor> FConversationEditor::CreateConversationEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* Object)
{
	TSharedRef<FConversationEditor> NewConversationEditor(new FConversationEditor());
	NewConversationEditor->InitConversationEditor(Mode, InitToolkitHost, Object);
	return NewConversationEditor;
}

void FConversationEditor::PostUndo(bool bSuccess)
{
	FAIGraphEditor::PostUndo(bSuccess);
}

void FConversationEditor::PostRedo(bool bSuccess)
{
	FAIGraphEditor::PostRedo(bSuccess);
}

void FConversationEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
}

void FConversationEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	DocumentManager->SetTabManager(InTabManager);

	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);
}

void FConversationEditor::RegisterToolbarTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FConversationEditor::InitConversationEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* InObject )
{
	UConversationDatabase* ConversationToEdit = Cast<UConversationDatabase>(InObject);

	if (ConversationToEdit != nullptr)
	{
		ConversationAsset = ConversationToEdit;
	}

	TSharedPtr<FConversationEditor> ThisPtr(SharedThis(this));
	if (!DocumentManager.IsValid())
	{
		DocumentManager = MakeShareable(new FDocumentTracker);
		DocumentManager->Initialize(ThisPtr);

		// Register the document factories
		{
			TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShareable(new FConversationGraphEditorSummoner(ThisPtr,
				FConversationGraphEditorSummoner::FOnCreateGraphEditorWidget::CreateSP(this, &FConversationEditor::CreateGraphEditorWidget)
			));

			// Also store off a reference to the grapheditor factory so we can find all the tabs spawned by it later.
			GraphEditorTabFactoryPtr = GraphEditorFactory;
			DocumentManager->RegisterDocumentFactory(GraphEditorFactory);
		}
	}

	TArray<UObject*> ObjectsToEdit;
	if(ConversationAsset != nullptr)
	{
		ObjectsToEdit.Add(ConversationAsset);
	}

	if (!ToolbarBuilder.IsValid())
	{
		ToolbarBuilder = MakeShareable(new FConversationEditorToolbar(SharedThis(this)));
	}

	FGraphEditorCommands::Register();
	FConversationEditorCommonCommands::Register();
	FConversationDebuggerCommands::Register();

	const TSharedRef<FTabManager::FLayout> DummyLayout = FTabManager::NewLayout("NullLayout")->AddArea(FTabManager::NewPrimaryArea());
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	InitAssetEditor( Mode, InitToolkitHost, ConversationEditorAppIdentifier, DummyLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit );

	BindCommonCommands();
	ExtendMenu();
	CreateInternalWidgets();

	Debugger = MakeShareable(new FConversationDebugger);
	Debugger->Setup(ConversationAsset, SharedThis(this));
	BindDebuggerToolbarCommands();

	AddApplicationMode(GraphViewMode, MakeShareable(new FConversationEditorApplicationMode_GraphView(SharedThis(this))));
	SetCurrentMode(GraphViewMode);

	OnClassListUpdated();
	RegenerateMenusAndToolbars();
}

void FConversationEditor::RestoreConversation()
{
	// Do graph
	{
		int32 NumGraphs = FConversationCompiler::GetNumGraphs(ConversationAsset);
		if (NumGraphs == 0)
		{
			FConversationCompiler::AddNewGraph(ConversationAsset, TEXT("ConversationGraph"));
			++NumGraphs;
		}

		for (int32 GraphIndex = 0; GraphIndex < NumGraphs; ++GraphIndex)
		{
			UConversationGraph* MyGraph = FConversationCompiler::GetGraphFromBank(ConversationAsset, GraphIndex);
			check(MyGraph);

			//@TODO: CONVERSATION: Hate this design
			MyGraph->OnLoaded();
			MyGraph->Initialize();
			MyGraph->UpdateAsset(/*UConversationGraph::KeepRebuildCounter*/0);
		}
	}

	// Make sure we always show a graph
	if (ConversationAsset->LastEditedDocuments.Num() == 0)
	{
		if (UConversationGraph* MyGraph = FConversationCompiler::GetGraphFromBank(ConversationAsset, 0))
		{
			ConversationAsset->LastEditedDocuments.Add(FEditedDocumentInfo(MyGraph));
		}
	}

	// Restore any previously opened graphs (or the newly added one)
	for (const FEditedDocumentInfo& DocumentInfo : ConversationAsset->LastEditedDocuments)
	{
		if (UObject* DocumentObject = DocumentInfo.EditedObjectPath.ResolveObject())
		{
			TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentObject);

			if (UConversationGraph* ConversationGraph = Cast<UConversationGraph>(DocumentObject))
			{
				TSharedPtr<SDockTab> DocumentTab = DocumentManager->OpenDocument(Payload, FDocumentTracker::RestorePreviousDocument);

				TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(DocumentTab->GetContent());
				GraphEditor->SetViewLocation(DocumentInfo.SavedViewOffset, DocumentInfo.SavedZoomAmount);
			}
			else
			{
				DocumentManager->OpenDocument(Payload, FDocumentTracker::RestorePreviousDocument);
			}
		}
	}

	FConversationCompiler::RebuildBank(ConversationAsset);
	RefreshDebugger();
}

void FConversationEditor::SaveEditedObjectState()
{
	// Clear currently edited documents
	ConversationAsset->LastEditedDocuments.Empty();

	// Ask all open documents to save their state, which will update LastEditedDocuments
	DocumentManager->SaveAllState();
}

FText FConversationEditor::HandleGetDebugKeyValue(const FName& InKeyName, bool bUseCurrentState) const
{
	if(IsDebuggerReady())
	{
		return Debugger->FindValueForKey(InKeyName, bUseCurrentState);
	}

	return FText();
}

float FConversationEditor::HandleGetDebugTimeStamp(bool bUseCurrentState) const
{
	if(IsDebuggerReady())
	{
		return Debugger->GetTimeStamp(bUseCurrentState);
	}

	return 0.0f;
}

bool FConversationEditor::IsDebuggerReady() const
{
	return Debugger.IsValid() && Debugger->IsDebuggerReady();
}

bool FConversationEditor::IsDebuggerPaused() const
{
	return IsDebuggerReady() && GUnrealEd->PlayWorld && GUnrealEd->PlayWorld->bDebugPauseExecution;
}

EVisibility FConversationEditor::GetDebuggerDetailsVisibility() const
{
	return Debugger.IsValid() && Debugger->IsDebuggerRunning() ? EVisibility::Visible : EVisibility::Collapsed;
}

FGraphAppearanceInfo FConversationEditor::GetGraphAppearance() const
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText", "CONVERSATION");

	const int32 StepIdx = Debugger.IsValid() ? Debugger->GetShownStateIndex() : 0;
	if (Debugger.IsValid() && !Debugger->IsDebuggerRunning())
	{
		AppearanceInfo.PIENotifyText = LOCTEXT("InactiveLabel", "INACTIVE");
	}
	else if (StepIdx)
	{
		AppearanceInfo.PIENotifyText = FText::Format(LOCTEXT("StepsBackLabelFmt", "STEPS BACK: {0}"), FText::AsNumber(StepIdx));
	}
	else if (FConversationDebugger::IsPlaySessionPaused())
	{
		AppearanceInfo.PIENotifyText = LOCTEXT("PausedLabel", "PAUSED");
	}
	
	return AppearanceInfo;
}

FName FConversationEditor::GetToolkitFName() const
{
	return FName("Conversation Editor");
}

FText FConversationEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Conversation");
}

FString FConversationEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Conversation ").ToString();
}

FLinearColor FConversationEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

TSharedRef<SGraphEditor> FConversationEditor::CreateGraphEditorWidget(UEdGraph* InGraph)
{
	check(InGraph != NULL);
	
	if (!GraphEditorCommands.IsValid())
	{
		CreateCommandList();

		// Debug actions
		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddBreakpoint,
			FExecuteAction::CreateSP( this, &FConversationEditor::OnAddBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FConversationEditor::CanAddBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FConversationEditor::CanAddBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().RemoveBreakpoint,
			FExecuteAction::CreateSP( this, &FConversationEditor::OnRemoveBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FConversationEditor::CanRemoveBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FConversationEditor::CanRemoveBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().EnableBreakpoint,
			FExecuteAction::CreateSP( this, &FConversationEditor::OnEnableBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FConversationEditor::CanEnableBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FConversationEditor::CanEnableBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DisableBreakpoint,
			FExecuteAction::CreateSP( this, &FConversationEditor::OnDisableBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FConversationEditor::CanDisableBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FConversationEditor::CanDisableBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().ToggleBreakpoint,
			FExecuteAction::CreateSP( this, &FConversationEditor::OnToggleBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FConversationEditor::CanToggleBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FConversationEditor::CanToggleBreakpoint )
			);
	}

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FConversationEditor::OnSelectedNodesChanged);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FConversationEditor::OnNodeDoubleClicked);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FConversationEditor::OnNodeTitleCommitted);

	// Make title bar
	TSharedRef<SWidget> TitleBarWidget = 
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush( TEXT("Graph.TitleBackground") ) )
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ConversationGraphLabel", "Conversation Editor"))
				.TextStyle( FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText") )
			]
		];

	// Make full graph editor
	const bool bGraphIsEditable = InGraph->bEditable;
	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(this, &FConversationEditor::InEditingMode, bGraphIsEditable)
		.Appearance(this, &FConversationEditor::GetGraphAppearance)
		.TitleBar(TitleBarWidget)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents);
}

bool FConversationEditor::InEditingMode(bool bGraphIsEditable) const
{
	return bGraphIsEditable && FConversationDebugger::IsPIENotSimulating();
}

TSharedRef<SWidget> FConversationEditor::SpawnSearch()
{
	FindResults = SNew(SFindInConversation, SharedThis(this));
	return FindResults.ToSharedRef();
}

TSharedRef<SWidget> FConversationEditor::SpawnProperties()
{
	return
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.HAlign(HAlign_Fill)
		[
			DetailsView.ToSharedRef()
		];
}

TSharedRef<SWidget> FConversationEditor::SpawnConversationTree()
{
	TreeEditor = SNew(SConversationTreeEditor, SharedThis(this));
	return TreeEditor.ToSharedRef();
}

void FConversationEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	ConversationEditorUtils::FPropertySelectionInfo SelectionInfo;
	TArray<UObject*> Selection = ConversationEditorUtils::GetSelectionForPropertyEditor(NewSelection, SelectionInfo);

	bForceDisablePropertyEdit = false;

	if (DetailsView.IsValid())
	{
		if (Selection.Num() == 0)
		{
			// if nothing is selected, display the bank settings itself
			DetailsView->SetObject(GetConversationAsset());
		}
		else if (Selection.Num() == 1)
		{
			DetailsView->SetObjects(Selection);
		}
		else
		{
			//@TODO: CONVERSATION: Support some useful form of multiselect
			DetailsView->SetObject(nullptr);
		}
	}
}

void FConversationEditor::CreateInternalWidgets()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject( nullptr );
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &FConversationEditor::IsPropertyEditable));
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FConversationEditor::OnFinishedChangingProperties);

	//DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ConversationNodeHandle"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConversationNodeReferenceCustomization::MakeInstance, TWeakPtr<FConversationEditor>(SharedThis(this))), nullptr);
}

void FConversationEditor::ExtendMenu()
{
	struct Local
	{
		static void FillEditMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("EditSearch", LOCTEXT("EditMenu_SearchHeading", "Search"));
			{
				MenuBuilder.AddMenuEntry(FConversationEditorCommonCommands::Get().SearchConversation);
			}
			MenuBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> MenuExtender = MakeShareable(new FExtender);

	// Extend the Edit menu
	MenuExtender->AddMenuExtension(
		"EditHistory",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateStatic(&Local::FillEditMenu));

	AddMenuExtender(MenuExtender);
}

void FConversationEditor::BindCommonCommands()
{
	ToolkitCommands->MapAction(FConversationEditorCommonCommands::Get().SearchConversation,
			FExecuteAction::CreateSP(this, &FConversationEditor::SearchConversationDatabase),
			FCanExecuteAction::CreateSP(this, &FConversationEditor::CanSearchConversationDatabase)
			);
}

void FConversationEditor::SearchConversationDatabase()
{
	TabManager->TryInvokeTab(FConversationEditorTabs::SearchID);
	FindResults->FocusForUse();
}

bool FConversationEditor::CanSearchConversationDatabase() const
{
	return true;
}

TSharedRef<class SWidget> FConversationEditor::OnGetDebuggerActorsMenu()
{
	FMenuBuilder MenuBuilder(true, NULL);

	if (Debugger.IsValid())
	{
// 		TArray<UConversationComponent*> MatchingInstances;
// 		Debugger->GetMatchingInstances(MatchingInstances);

		// Fill the combo menu with presets of common screen resolutions
		//@TODO: CONVERSATION: debugging stuff
// 		for (int32 i = 0; i < MatchingInstances.Num(); i++)
// 		{
// 			if (MatchingInstances[i])
// 			{
// 				const FText ActorDesc = FText::FromString(Debugger->DescribeInstance(*MatchingInstances[i]));
// 				TWeakObjectPtr<UConversationComponent> InstancePtr = MatchingInstances[i];
// 
// 				FUIAction ItemAction(FExecuteAction::CreateSP(this, &FConversationEditor::OnDebuggerActorSelected, InstancePtr));
// 				MenuBuilder.AddMenuEntry(ActorDesc, TAttribute<FText>(), FSlateIcon(), ItemAction);
// 			}
// 		}

		// Failsafe when no actor match
// 		if (MatchingInstances.Num() == 0)
// 		{
// 			const FText ActorDesc = LOCTEXT("NoMatchForDebug","Can't find matching actors");
// 			TWeakObjectPtr<UConversationComponent> InstancePtr;
// 
// 			FUIAction ItemAction(FExecuteAction::CreateSP(this, &FConversationEditor::OnDebuggerActorSelected, InstancePtr));
// 			MenuBuilder.AddMenuEntry(ActorDesc, TAttribute<FText>(), FSlateIcon(), ItemAction);
// 		}
	}

	return MenuBuilder.MakeWidget();
}

// void FConversationEditor::OnDebuggerActorSelected(TWeakObjectPtr<UConversationComponent> InstanceToDebug)
// {
// 	if (Debugger.IsValid())
// 	{
// 		Debugger->OnInstanceSelectedInDropdown(InstanceToDebug.Get());
// 	}
//}

FText FConversationEditor::GetDebuggerActorDesc() const
{
	return Debugger.IsValid() ? FText::FromString(Debugger->GetDebuggedInstanceDesc()) : FText::GetEmpty();
}

void FConversationEditor::BindDebuggerToolbarCommands()
{
	const FConversationDebuggerCommands& Commands = FConversationDebuggerCommands::Get();
	TSharedRef<FConversationDebugger> DebuggerOb = Debugger.ToSharedRef();

	ToolkitCommands->MapAction(
		Commands.BackOver,
		FExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::StepBackOver),
		FCanExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::CanStepBackOver));

	ToolkitCommands->MapAction(
		Commands.BackInto,
		FExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::StepBackInto),
		FCanExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::CanStepBackInto));

	ToolkitCommands->MapAction(
		Commands.ForwardInto,
		FExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::StepForwardInto),
		FCanExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::CanStepForwardInto));

	ToolkitCommands->MapAction(
		Commands.ForwardOver,
		FExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::StepForwardOver),
		FCanExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::CanStepForwardOver));

	ToolkitCommands->MapAction(
		Commands.StepOut,
		FExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::StepOut),
		FCanExecuteAction::CreateSP(DebuggerOb, &FConversationDebugger::CanStepOut));

	ToolkitCommands->MapAction(
		Commands.PausePlaySession,
		FExecuteAction::CreateStatic(&FConversationDebugger::PausePlaySession),
		FCanExecuteAction::CreateStatic(&FConversationDebugger::IsPlaySessionRunning),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FConversationDebugger::IsPlaySessionRunning));

	ToolkitCommands->MapAction(
		Commands.ResumePlaySession,
		FExecuteAction::CreateStatic(&FConversationDebugger::ResumePlaySession),
		FCanExecuteAction::CreateStatic(&FConversationDebugger::IsPlaySessionPaused),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FConversationDebugger::IsPlaySessionPaused));

	ToolkitCommands->MapAction(
		Commands.StopPlaySession,
		FExecuteAction::CreateStatic(&FConversationDebugger::StopPlaySession));
}

bool FConversationEditor::IsPropertyEditable() const
{
	if (FConversationDebugger::IsPIESimulating() || bForceDisablePropertyEdit)
	{
		return false;
	}

	TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin();
	return FocusedGraphEd.IsValid() && FocusedGraphEd->GetCurrentGraph() && FocusedGraphEd->GetCurrentGraph()->bEditable;
}

void FConversationEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid() && FocusedGraphEd->GetCurrentGraph())
	{
		FocusedGraphEd->GetCurrentGraph()->GetSchema()->ForceVisualizationCacheClear();
	}

	//@TODO: CONVERSATION: Should figure out how to do this only when necessary instead of always...
	FConversationCompiler::RebuildBank(ConversationAsset);
}

void FConversationEditor::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	if (Node && Node->CanJumpToDefinition())
	{
		Node->JumpToDefinition();
	}
}

void FConversationEditor::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	if (UpdateGraphEdPtr.Pin() != InGraphEditor)
	{
		UpdateGraphEdPtr = InGraphEditor;
		FocusedGraphEditorChanged.Broadcast();
	}

	FGraphPanelSelectionSet CurrentSelection;
	CurrentSelection = InGraphEditor->GetSelectedNodes();
	OnSelectedNodesChanged(CurrentSelection);
}

void FConversationEditor::OnEnableBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && HasBreakpoint(SelectedNode) && !HasEnabledBreakpoint(SelectedNode))
		{
			SetBreakpointEnabled(SelectedNode, true);
			Debugger->OnBreakpointAdded(SelectedNode);
		}
	}
}

bool FConversationEditor::CanEnableBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && HasBreakpoint(SelectedNode) && !HasEnabledBreakpoint(SelectedNode))
		{
			return true;
		}
	}

	return false;
}

void FConversationEditor::OnDisableBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && HasBreakpoint(SelectedNode) && HasEnabledBreakpoint(SelectedNode))
		{
			SetBreakpointEnabled(SelectedNode, false);
			Debugger->OnBreakpointRemoved(SelectedNode);
		}
	}
}

bool FConversationEditor::CanDisableBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && HasBreakpoint(SelectedNode) && HasEnabledBreakpoint(SelectedNode))
		{
			return true;
		}
	}

	return false;
}

void FConversationEditor::OnAddBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && !HasBreakpoint(SelectedNode))
		{
			SetBreakpointEnabled(SelectedNode, true);
			Debugger->OnBreakpointAdded(SelectedNode);
		}
	}
}

bool FConversationEditor::CanAddBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && !HasBreakpoint(SelectedNode))
		{
			return true;
		}
	}

	return false;
}

void FConversationEditor::OnRemoveBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && HasBreakpoint(SelectedNode))
		{
			RemoveBreakpoint(SelectedNode);
			Debugger->OnBreakpointRemoved(SelectedNode);
		}
	}
}

bool FConversationEditor::CanRemoveBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && HasBreakpoint(SelectedNode))
		{
			return true;
		}
	}

	return false;
}

void FConversationEditor::OnToggleBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints())
		{
			if (HasBreakpoint(SelectedNode))
			{
				RemoveBreakpoint(SelectedNode);
				Debugger->OnBreakpointRemoved(SelectedNode);
			}
			else
			{
				SetBreakpointEnabled(SelectedNode, true);
				Debugger->OnBreakpointAdded(SelectedNode);
			}
		}
	}
}

bool FConversationEditor::CanToggleBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UConversationGraphNode* SelectedNode = Cast<UConversationGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints())
		{
			return true;
		}
	}

	return false;
}

void FConversationEditor::JumpToNode(const UEdGraphNode* Node)
{
	TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
	if (ActiveTab.IsValid())
	{
		TSharedPtr<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
		if (GraphEditor.IsValid())
		{
			GraphEditor->JumpToNode(Node, false);
		}
	}
}

TWeakPtr<SGraphEditor> FConversationEditor::GetFocusedGraphPtr() const
{
	return UpdateGraphEdPtr;
}

FText FConversationEditor::GetLocalizedMode(FName InMode)
{
	static TMap< FName, FText > LocModes;

	if (LocModes.Num() == 0)
	{
		LocModes.Add(GraphViewMode, LOCTEXT("GraphVieWMode", "Graph") );
		LocModes.Add(TreeViewMode, LOCTEXT("TreeViewMode", "Tree View") );
	}

	check( InMode != NAME_None );
	const FText* OutDesc = LocModes.Find( InMode );
	check( OutDesc );
	return *OutDesc;
}

void FConversationEditor::FocusWindow(UObject* ObjectToFocusOn)
{
	FWorkflowCentricApplication::FocusWindow(ObjectToFocusOn);
}

void FConversationEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction(LOCTEXT("ConversationRenameNode", "Rename Node"));
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

FText FConversationEditor::GetToolkitName() const
{
	return ConversationAsset ? FAssetEditorToolkit::GetLabelForObject(ConversationAsset) : FText();
}

FText FConversationEditor::GetToolkitToolTipText() const
{
	return ConversationAsset ? FAssetEditorToolkit::GetToolTipTextForObject(ConversationAsset) : FText();
}

UConversationDatabase* FConversationEditor::GetConversationAsset() const
{
	return ConversationAsset;
}

void FConversationEditor::RefreshDebugger()
{
	Debugger->Refresh();
}

bool FConversationEditor::CanCreateNewNodeClasses() const
{
	return !IsDebuggerReady();
}

TSharedRef<SWidget> FConversationEditor::HandleCreateNewClassMenu(UClass* BaseClass) const
{
	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.ClassFilters.Add(MakeShareable( new FNewNodeClassFilter(BaseClass) ));

	FOnClassPicked OnPicked( FOnClassPicked::CreateSP( this, &FConversationEditor::HandleNewNodeClassPicked ) );

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

void FConversationEditor::HandleNewNodeClassPicked(UClass* InClass) const
{
	if (ConversationAsset != nullptr)
	{
		const FString ClassName = FBlueprintEditorUtils::GetClassNameWithoutSuffix(InClass);

		FString PathName = ConversationAsset->GetOutermost()->GetPathName();
		PathName = FPaths::GetPath(PathName);

		// Now that we've generated some reasonable default locations/names for the package, allow the user to have the final say
		// before we create the package and initialize the blueprint inside of it.
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
		SaveAssetDialogConfig.DefaultPath = PathName;
		SaveAssetDialogConfig.DefaultAssetName = ClassName + TEXT("_New");
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

		const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (!SaveObjectPath.IsEmpty())
		{
			const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			const FString SavePackagePath = FPaths::GetPath(SavePackageName);
			const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);

			UPackage* Package = CreatePackage(*SavePackageName);
			if (ensure(Package))
			{
				// Create and init a new Blueprint
				if (UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(InClass, Package, FName(*SaveAssetName), BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass()))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(NewBP);

					// Mark the package dirty...
					Package->MarkPackageDirty();
				}
			}
		}
	}
}

void FConversationEditor::FixupPastedNodes(const TSet<UEdGraphNode*>& PastedGraphNodes, const TMap<FGuid/*New*/, FGuid/*Old*/>& NewToOldNodeMapping)
{
	FAIGraphEditor::FixupPastedNodes(PastedGraphNodes, NewToOldNodeMapping);


}

#undef LOCTEXT_NAMESPACE
