// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "AIGraphEditor.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class SGraphEditor;
struct FGraphAppearanceInfo;

class FConversationDebugger;
class FConversationEditorToolbar;
class FDocumentTabFactory;
class FDocumentTracker;
class IDetailsView;
class SFindInConversation;
class UConversation;
class UEdGraph;
struct Rect;
class SConversationTreeEditor;

class UEdGraphNode;
class UConversationDatabase;

class FConversationEditor : public FWorkflowCentricApplication, public FAIGraphEditor, public FNotifyHook
{
public:
	FConversationEditor();
	/** Destructor */
	virtual ~FConversationEditor();

	FSimpleMulticastDelegate FocusedGraphEditorChanged;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	void InitConversationEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* InObject );

	static TSharedRef<FConversationEditor> CreateConversationEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* Object);

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	//~ End IToolkit Interface

	virtual void FocusWindow(UObject* ObjectToFocusOn = NULL);

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	//~ Begin FNotifyHook Interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	// End of FNotifyHook

	// Delegates
	void OnNodeDoubleClicked(class UEdGraphNode* Node);
	void OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor);
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	void OnEnableBreakpoint();
	bool CanEnableBreakpoint() const;
	void OnToggleBreakpoint();
	bool CanToggleBreakpoint() const;
	void OnDisableBreakpoint();
	bool CanDisableBreakpoint() const;
	void OnAddBreakpoint();
	bool CanAddBreakpoint() const;
	void OnRemoveBreakpoint();
	bool CanRemoveBreakpoint() const;

	void SearchConversationDatabase();
	bool CanSearchConversationDatabase() const;

	void JumpToNode(const UEdGraphNode* Node);

	bool IsPropertyEditable() const;
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	void UpdateToolbar();
	bool IsDebuggerReady() const;

	/** Get whether the debugger is currently running and the PIE session is paused */
	bool IsDebuggerPaused() const;

	TSharedRef<class SWidget> OnGetDebuggerActorsMenu();
//	void OnDebuggerActorSelected(TWeakObjectPtr<UConversationComponent> InstanceToDebug);
	FText GetDebuggerActorDesc() const;

	FGraphAppearanceInfo GetGraphAppearance() const;
	bool InEditingMode(bool bGraphIsEditable) const;

	EVisibility GetDebuggerDetailsVisibility() const;
	EVisibility GetInjectedNodeVisibility() const;

	TWeakPtr<SGraphEditor> GetFocusedGraphPtr() const;

	/** 
	 * Get the localized text to display for the specified mode 
	 * @param	InMode	The mode to display
	 * @return the localized text representation of the mode
	 */
	static FText GetLocalizedMode(FName InMode);

	/** Access the toolbar builder for this editor */
	TSharedPtr<class FConversationEditorToolbar> GetToolbarBuilder() { return ToolbarBuilder; }

	/** Get the behavior tree we are editing (if any) */
	UConversationDatabase* GetConversationAsset() const;

	/** Spawns the tab with the update graph inside */
	TSharedRef<SWidget> SpawnProperties();

	/** Spawns the search tab */
	TSharedRef<SWidget> SpawnSearch();

	/** Spawns the conversation tree tab */
	TSharedRef<SWidget> SpawnConversationTree();

	// @todo This is a hack for now until we reconcile the default toolbar with application modes [duplicated from counterpart in Blueprint Editor]
	void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);

	/** Restores the behavior tree graph we were editing or creates a new one if none is available */
	void RestoreConversation();

	/** Save the graph state for later editing */
	void SaveEditedObjectState();

	/** Delegate handler for displaying debugger values */
	FText HandleGetDebugKeyValue(const FName& InKeyName, bool bUseCurrentState) const;

	/** Delegate handler for retrieving timestamp to display */
	float HandleGetDebugTimeStamp(bool bUseCurrentState) const;

	/** Are we allowed to create new node classes right now? */
	bool CanCreateNewNodeClasses() const;

	/** Create the menu used to make a new task node */
	TSharedRef<SWidget> HandleCreateNewClassMenu(UClass* BaseClass) const;

	/** Handler for when a node class is picked */
	void HandleNewNodeClassPicked(UClass* InClass) const;

protected:
	virtual void FixupPastedNodes(const TSet<UEdGraphNode*>& PastedGraphNodes, const TMap<FGuid/*New*/, FGuid/*Old*/>& NewToOldNodeMapping) override;

private:
	/** Create widget for graph editing */
	TSharedRef<class SGraphEditor> CreateGraphEditorWidget(UEdGraph* InGraph);

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Add custom menu options */
	void ExtendMenu();

	/** Setup common commands */
	void BindCommonCommands();

	/** Setup commands */
	void BindDebuggerToolbarCommands();

	/** Called when the selection changes in the GraphEditor */
	virtual void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection) override;

	/** Refresh the debugger's display */
	void RefreshDebugger();

	TSharedPtr<FDocumentTracker> DocumentManager;
	TWeakPtr<FDocumentTabFactory> GraphEditorTabFactoryPtr;

	/* The conversation being edited */
	UConversationDatabase* ConversationAsset;

	/** Property View */
	TSharedPtr<class IDetailsView> DetailsView;

	TSharedPtr<class FConversationDebugger> Debugger;

	/** Find results log as well as the search filter */
	TSharedPtr<SFindInConversation> FindResults;

	TSharedPtr<SConversationTreeEditor> TreeEditor;

	uint32 bForceDisablePropertyEdit : 1;

	TSharedPtr<FConversationEditorToolbar> ToolbarBuilder;

public:
	/** Modes in mode switcher */
	static const FName GraphViewMode;
	static const FName TreeViewMode;
};
