// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditorCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ConversationEditorCommands"

//////////////////////////////////////////////////////////////////////
//

FConversationEditorCommonCommands::FConversationEditorCommonCommands()
	: TCommands<FConversationEditorCommonCommands>("ConversationEditor.Common", LOCTEXT("ComversationEditorCommandsLabel", "Conversation Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FConversationEditorCommonCommands::RegisterCommands()
{
	UI_COMMAND(SearchConversation, "Search", "Search this conversation bank.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
}

//////////////////////////////////////////////////////////////////////
//

FConversationDebuggerCommands::FConversationDebuggerCommands()
	: TCommands<FConversationDebuggerCommands>("ConversationEditor.Debugger", LOCTEXT("Debugger", "Debugger"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FConversationDebuggerCommands::RegisterCommands()
{
	UI_COMMAND(BackInto, "Back: Into", "Show state from previous step, can go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BackOver, "Back: Over", "Show state from previous step, don't go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ForwardInto, "Forward: Into", "Show state from next step, can go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ForwardOver, "Forward: Over", "Show state from next step, don't go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StepOut, "Step Out", "Show state from next step, leave current subtree", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(PausePlaySession, "Pause", "Pause simulation", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResumePlaySession, "Resume", "Resume simulation", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND(StopPlaySession, "Stop", "Stop simulation", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CurrentValues, "Current", "View current values", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SavedValues, "Saved", "View saved values", EUserInterfaceActionType::RadioButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
