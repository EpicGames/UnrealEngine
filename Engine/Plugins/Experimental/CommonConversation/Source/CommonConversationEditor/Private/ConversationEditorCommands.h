// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

//////////////////////////////////////////////////////////////////////
//

class FConversationEditorCommonCommands : public TCommands<FConversationEditorCommonCommands>
{
public:
	FConversationEditorCommonCommands();

	TSharedPtr<FUICommandInfo> SearchConversation;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

//////////////////////////////////////////////////////////////////////
//

class FConversationDebuggerCommands : public TCommands<FConversationDebuggerCommands>
{
public:
	FConversationDebuggerCommands();

	TSharedPtr<FUICommandInfo> BackInto;
	TSharedPtr<FUICommandInfo> BackOver;
	TSharedPtr<FUICommandInfo> ForwardInto;
	TSharedPtr<FUICommandInfo> ForwardOver;
	TSharedPtr<FUICommandInfo> StepOut;

	TSharedPtr<FUICommandInfo> PausePlaySession;
	TSharedPtr<FUICommandInfo> ResumePlaySession;
	TSharedPtr<FUICommandInfo> StopPlaySession;

	TSharedPtr<FUICommandInfo> CurrentValues;
	TSharedPtr<FUICommandInfo> SavedValues;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

