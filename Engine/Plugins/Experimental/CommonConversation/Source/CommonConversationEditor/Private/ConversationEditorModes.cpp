// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditorModes.h"
#include "ConversationEditor.h"
#include "ConversationEditorTabs.h"
#include "ConversationEditorTabFactories.h"
#include "ConversationEditorToolbar.h"

//////////////////////////////////////////////////////////////////////
// FConversationEditorApplicationMode_GraphView

FConversationEditorApplicationMode_GraphView::FConversationEditorApplicationMode_GraphView(TSharedPtr<FConversationEditor> InConversationEditor)
	: FApplicationMode(FConversationEditor::GraphViewMode, FConversationEditor::GetLocalizedMode)
{
	ConversationEditor = InConversationEditor;

	ConversationEditorTabFactories.RegisterFactory(MakeShareable(new FConversationDetailsSummoner(InConversationEditor)));
	ConversationEditorTabFactories.RegisterFactory(MakeShareable(new FConversationSearchSummoner(InConversationEditor)));
	ConversationEditorTabFactories.RegisterFactory(MakeShareable(new FConversationTreeEditorSummoner(InConversationEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_ConversationEditor_GraphView_Layout_v4" )
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Horizontal)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.7f)
			->AddTab(FConversationEditorTabs::GraphEditorID, ETabState::ClosedTab)
		)
		->Split
		(
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Vertical)
			->SetSizeCoefficient(0.3f)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.6f)
				->AddTab(FConversationEditorTabs::GraphDetailsID, ETabState::OpenedTab)
				->AddTab(FConversationEditorTabs::SearchID, ETabState::ClosedTab)
			)
 			->Split
 			(
 				FTabManager::NewStack()
 				->SetSizeCoefficient(0.4f)
 				->AddTab(FConversationEditorTabs::TreeEditorID, ETabState::OpenedTab)
 			)
		)
	);
	
	InConversationEditor->GetToolbarBuilder()->AddDebuggerToolbar(ToolbarExtender);
	InConversationEditor->GetToolbarBuilder()->AddConversationEditorToolbar(ToolbarExtender);
}

void FConversationEditorApplicationMode_GraphView::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	check(ConversationEditor.IsValid());
	TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();
	
	ConversationEditorPtr->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	ConversationEditorPtr->PushTabFactories(ConversationEditorTabFactories);
}

void FConversationEditorApplicationMode_GraphView::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	check(ConversationEditor.IsValid());
	TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();
	
	ConversationEditorPtr->SaveEditedObjectState();
}

void FConversationEditorApplicationMode_GraphView::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	check(ConversationEditor.IsValid());
	TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();
	ConversationEditorPtr->RestoreConversation();

	FApplicationMode::PostActivateMode();
}
