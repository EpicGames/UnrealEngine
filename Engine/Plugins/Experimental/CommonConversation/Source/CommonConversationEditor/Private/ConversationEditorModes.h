// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

class FConversationEditor;

//////////////////////////////////////////////////////////////////////
//

/** Application mode for main conversation editing mode */
class FConversationEditorApplicationMode_GraphView : public FApplicationMode
{
public:
	FConversationEditorApplicationMode_GraphView(TSharedPtr<FConversationEditor> InConversationEditor);

	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PreDeactivateMode() override;
	virtual void PostActivateMode() override;

protected:
	TWeakPtr<FConversationEditor> ConversationEditor;

	// Set of spawnable tabs in behavior tree editing mode
	FWorkflowAllowedTabSet ConversationEditorTabFactories;
};
