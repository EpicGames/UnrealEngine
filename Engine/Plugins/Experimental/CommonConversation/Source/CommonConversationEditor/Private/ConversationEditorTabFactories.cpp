// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditorTabFactories.h"
#include "ConversationEditor.h"
#include "Engine/Blueprint.h"
#include "ConversationEditorTabs.h"
#include "Widgets/Docking/SDockTab.h"
#include "ConversationDatabase.h"

#define LOCTEXT_NAMESPACE "ConversationEditorFactories"

//////////////////////////////////////////////////////////////////////
//

FConversationDetailsSummoner::FConversationDetailsSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr)
	: FWorkflowTabFactory(FConversationEditorTabs::GraphDetailsID, InConversationEditorPtr)
	, ConversationEditorPtr(InConversationEditorPtr)
{
	TabLabel = LOCTEXT("ConversationDetailsLabel", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Components");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ConversationDetailsView", "Details");
	ViewMenuTooltip = LOCTEXT("ConversationDetailsView_ToolTip", "Show the details view");
}

TSharedRef<SWidget> FConversationDetailsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(ConversationEditorPtr.IsValid());
	return ConversationEditorPtr.Pin()->SpawnProperties();
}

FText FConversationDetailsSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("ConversationDetailsTabTooltip", "The behavior tree details tab allows editing of the properties of behavior tree nodes");
}

//////////////////////////////////////////////////////////////////////
//

FConversationSearchSummoner::FConversationSearchSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr)
	: FWorkflowTabFactory(FConversationEditorTabs::SearchID, InConversationEditorPtr)
	, ConversationEditorPtr(InConversationEditorPtr)
{
	TabLabel = LOCTEXT("ConversationSearchLabel", "Search");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ConversationSearchView", "Search");
	ViewMenuTooltip = LOCTEXT("ConversationSearchView_ToolTip", "Show the behavior tree search tab");
}

TSharedRef<SWidget> FConversationSearchSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return ConversationEditorPtr.Pin()->SpawnSearch();
}

FText FConversationSearchSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("ConversationSearchTabTooltip", "The behavior tree search tab allows searching within behavior tree nodes");
}

//////////////////////////////////////////////////////////////////////
//

FConversationTreeEditorSummoner::FConversationTreeEditorSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr)
	: FWorkflowTabFactory(FConversationEditorTabs::TreeEditorID, InConversationEditorPtr)
	, ConversationEditorPtr(InConversationEditorPtr)
{
	TabLabel = LOCTEXT("ConversationTreeEditorLabel", "Conversation Tree");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Tree");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ConversationTreeEditorView", "Conversation Tree");
	ViewMenuTooltip = LOCTEXT("ConversationTreeEditorView_ToolTip", "Conversation Tree");
}

TSharedRef<SWidget> FConversationTreeEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return ConversationEditorPtr.Pin()->SpawnConversationTree();
}

FText FConversationTreeEditorSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return FText();
}

//////////////////////////////////////////////////////////////////////
//

FConversationGraphEditorSummoner::FConversationGraphEditorSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback)
	: FDocumentTabFactoryForObjects<UEdGraph>(FConversationEditorTabs::GraphEditorID, InConversationEditorPtr)
	, ConversationEditorPtr(InConversationEditorPtr)
	, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{
}

void FConversationGraphEditorSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	check(ConversationEditorPtr.IsValid());
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	ConversationEditorPtr.Pin()->OnGraphEditorFocused(GraphEditor);
}

void FConversationGraphEditorSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	GraphEditor->NotifyGraphChanged();
}

TAttribute<FText> FConversationGraphEditorSummoner::ConstructTabNameForObject(UEdGraph* DocumentID) const
{
	return TAttribute<FText>( FText::FromString( DocumentID->GetName() ) );
}

TSharedRef<SWidget> FConversationGraphEditorSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return OnCreateGraphEditorWidget.Execute(DocumentID);
}

const FSlateBrush* FConversationGraphEditorSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return FAppStyle::GetBrush("NoBrush");
}

void FConversationGraphEditorSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	check(ConversationEditorPtr.IsValid());
	check(ConversationEditorPtr.Pin()->GetConversationAsset());

	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());

	FVector2f ViewLocation;
	float ZoomAmount;
	GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);

	UEdGraph* Graph = FTabPayload_UObject::CastChecked<UEdGraph>(Payload);
	ConversationEditorPtr.Pin()->GetConversationAsset()->LastEditedDocuments.Add(FEditedDocumentInfo(Graph, ViewLocation, ZoomAmount));
}

#undef LOCTEXT_NAMESPACE
