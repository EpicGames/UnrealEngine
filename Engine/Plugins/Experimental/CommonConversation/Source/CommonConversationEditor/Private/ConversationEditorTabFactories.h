// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h" // IWYU pragma: keep
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

class SGraphEditor;

class FConversationEditor;

struct FConversationDetailsSummoner : public FWorkflowTabFactory
{
public:
	FConversationDetailsSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FConversationEditor> ConversationEditorPtr;
};

struct FConversationSearchSummoner : public FWorkflowTabFactory
{
public:
	FConversationSearchSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FConversationEditor> ConversationEditorPtr;
};

struct FConversationTreeEditorSummoner : public FWorkflowTabFactory
{
public:
	FConversationTreeEditorSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FConversationEditor> ConversationEditorPtr;
};

struct FConversationGraphEditorSummoner : public FDocumentTabFactoryForObjects<UEdGraph>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, UEdGraph*);
public:
	FConversationGraphEditorSummoner(TSharedPtr<FConversationEditor> InConversationEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;
	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;

protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UEdGraph* DocumentID) const override;
	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;
	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;
	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

protected:
	TWeakPtr<FConversationEditor> ConversationEditorPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};

