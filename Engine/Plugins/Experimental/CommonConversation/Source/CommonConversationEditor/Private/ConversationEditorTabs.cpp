// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditorTabs.h"
#include "UObject/NameTypes.h"

// Tab identifiers
const FName FConversationEditorTabs::GraphDetailsID(TEXT("ConversationEditor_Properties"));
const FName FConversationEditorTabs::SearchID(TEXT("ConversationEditor_Search"));
const FName FConversationEditorTabs::TreeEditorID(TEXT("ConversationEditor_Tree"));

// Document tab identifiers
const FName FConversationEditorTabs::GraphEditorID(TEXT("Document"));
