// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FName;

struct FConversationEditorTabs
{
	// Tab identifiers
	static const FName GraphDetailsID;
	static const FName SearchID;

	// Document tab identifiers
	static const FName GraphEditorID;
	static const FName TreeEditorID;
};

