// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditorToolbar.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ConversationEditor.h"
#include "ConversationEditorCommands.h"


#include "ConversationTaskNode.h"
#include "ConversationSideEffectNode.h"
#include "ConversationRequirementNode.h"


#define LOCTEXT_NAMESPACE "ConversationEditorToolbar"

class SConversationModeSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SConversationModeSeparator) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
			);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const float Height = 20.0f;
		const float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};

void FConversationEditorToolbar::AddDebuggerToolbar(TSharedPtr<FExtender> Extender)
{
	// setup toolbar
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, TWeakPtr<FConversationEditor> ConversationEditor)
		{
			TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();

			const bool bCanShowDebugger = ConversationEditorPtr->IsDebuggerReady();
			if (bCanShowDebugger)
			{
				TSharedRef<SWidget> SelectionBox = SNew(SComboButton)
					.OnGetMenuContent(ConversationEditorPtr.Get(), &FConversationEditor::OnGetDebuggerActorsMenu )
					.ButtonContent()
					[
						SNew(STextBlock)
						.ToolTipText( LOCTEXT("SelectDebugActor", "Pick actor to debug") )
						.Text(ConversationEditorPtr.Get(), &FConversationEditor::GetDebuggerActorDesc )
					];

				ToolbarBuilder.BeginSection("CachedState");
				{
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().BackOver);
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().BackInto);
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().ForwardInto);
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().ForwardOver);
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().StepOut);
				}
				ToolbarBuilder.EndSection();
				ToolbarBuilder.BeginSection("World");
				{
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().PausePlaySession);
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().ResumePlaySession);
					ToolbarBuilder.AddToolBarButton(FConversationDebuggerCommands::Get().StopPlaySession);
					ToolbarBuilder.AddSeparator();
					ToolbarBuilder.AddWidget(SelectionBox);
				}
				ToolbarBuilder.EndSection();
			}
		}
	};

	TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, ConversationEditorPtr->GetToolkitCommands(), FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar, ConversationEditor ));
	ConversationEditorPtr->AddToolbarExtender(ToolbarExtender);
}


void FConversationEditorToolbar::AddConversationEditorToolbar(TSharedPtr<FExtender> Extender)
{
	check(ConversationEditor.IsValid());
	TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, ConversationEditorPtr->GetToolkitCommands(), FToolBarExtensionDelegate::CreateSP( this, &FConversationEditorToolbar::FillConversationEditorToolbar ));
	ConversationEditorPtr->AddToolbarExtender(ToolbarExtender);
}

void FConversationEditorToolbar::FillConversationEditorToolbar(FToolBarBuilder& ToolbarBuilder)
{
	check(ConversationEditor.IsValid());
	TSharedPtr<FConversationEditor> ConversationEditorPtr = ConversationEditor.Pin();

	if (!ConversationEditorPtr->IsDebuggerReady() && ConversationEditorPtr->GetCurrentMode() == FConversationEditor::GraphViewMode)
	{
// 		ToolbarBuilder.BeginSection("Blackboard");
// 		{
// 			ToolbarBuilder.AddToolBarButton(FBTCommonCommands::Get().NewBlackboard);
// 		}
// 		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginSection("Conversation");
		{
			ToolbarBuilder.AddComboButton(
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP(ConversationEditorPtr.Get(), &FConversationEditor::CanCreateNewNodeClasses),
					FIsActionChecked()
					), 
				FOnGetContent::CreateSP(ConversationEditorPtr.Get(), &FConversationEditor::HandleCreateNewClassMenu, UConversationTaskNode::StaticClass()),
				LOCTEXT("NewTask_Label", "New Task"),
				LOCTEXT("NewTask_ToolTip", "Create a new task node Blueprint from a base class"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Graph.NewTask")
			);

			ToolbarBuilder.AddComboButton(
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP(ConversationEditorPtr.Get(), &FConversationEditor::CanCreateNewNodeClasses),
					FIsActionChecked()
					), 
				FOnGetContent::CreateSP(ConversationEditorPtr.Get(), &FConversationEditor::HandleCreateNewClassMenu, UConversationRequirementNode::StaticClass()),
				LOCTEXT("NewRequirement_Label", "New Requirement"),
				LOCTEXT("NewRequirement_ToolTip", "Create a new requirement node Blueprint from a base class"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Graph.NewDecorator")
			);

			ToolbarBuilder.AddComboButton(
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP(ConversationEditorPtr.Get(), &FConversationEditor::CanCreateNewNodeClasses),
					FIsActionChecked()
					), 
				FOnGetContent::CreateSP(ConversationEditorPtr.Get(), &FConversationEditor::HandleCreateNewClassMenu, UConversationSideEffectNode::StaticClass()),
				LOCTEXT("NewSideEffect_Label", "New Side Effect"),
				LOCTEXT("NewSideEffect_ToolTip", "Create a new side effect node Blueprint from a base class"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Graph.NewService")
			);
		}
		ToolbarBuilder.EndSection();
	}
}

#undef LOCTEXT_NAMESPACE
