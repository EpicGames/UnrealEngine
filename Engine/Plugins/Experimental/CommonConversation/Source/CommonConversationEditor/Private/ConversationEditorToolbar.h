// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FConversationEditor;
class FConversationTreeEditor;
class FExtender;
class FToolBarBuilder;

class FConversationEditorToolbar : public TSharedFromThis<FConversationEditorToolbar>
{
public:
	FConversationEditorToolbar(TSharedPtr<FConversationEditor> InConversationEditor)
		: ConversationEditor(InConversationEditor) {}

	void AddDebuggerToolbar(TSharedPtr<FExtender> Extender);
	void AddConversationEditorToolbar(TSharedPtr<FExtender> Extender);

private:
	void FillDebuggerToolbar(FToolBarBuilder& ToolbarBuilder);
	void FillConversationEditorToolbar(FToolBarBuilder& ToolbarBuilder);

protected:
	/** Pointer back to the blueprint editor tool that owns us */
	TWeakPtr<FConversationEditor> ConversationEditor;
};
