// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEditorUtils.h"
#include "ConversationGraphNode.h"

TArray<UObject*> ConversationEditorUtils::GetSelectionForPropertyEditor(const TSet<UObject*>& InSelection, FPropertySelectionInfo& OutSelectionInfo)
{
	TArray<UObject*> Selection;

	for (UObject* SelectionEntry : InSelection)
	{
		if (UConversationGraphNode* ConversationNode = Cast<UConversationGraphNode>(SelectionEntry))
		{
			Selection.Add(ConversationNode->NodeInstance);
			continue;
		}

		Selection.Add(SelectionEntry);
	}

	return Selection;
}
