// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"

namespace ConversationEditorUtils
{
	struct FPropertySelectionInfo
	{
		FPropertySelectionInfo()
		{
		}
	};

	/** Given a selection of nodes, return the instances that should be selected be selected for editing in the property panel */
	TArray<UObject*> GetSelectionForPropertyEditor(const TSet<UObject*>& InSelection, FPropertySelectionInfo& OutSelectionInfo);
}
