// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInConversationGraph.h"
#include "ConversationEditor.h"
#include "EdGraph/EdGraph.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "ConversationGraphNode.h"
//#include "ConversationGraphNode_Decorator.h"
//#include "ConversationGraphNode_Service.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "FindInBT"

//////////////////////////////////////////////////////////////////////////
// FFindInConversationResult

FFindInConversationResult::FFindInConversationResult(const FString& InValue)
	: Value(InValue), GraphNode(NULL)
{
}

FFindInConversationResult::FFindInConversationResult(const FString& InValue, TSharedPtr<FFindInConversationResult>& InParent, UEdGraphNode* InNode)
	: Value(InValue), GraphNode(InNode), Parent(InParent)
{
}

void FFindInConversationResult::SetNodeHighlight(bool bHighlight)
{
// 	if (GraphNode.IsValid())
// 	{
// 		UConversationGraphNode* BTNode = Cast<UConversationGraphNode>(GraphNode.Get());
// 		if (BTNode)
// 		{
// 			BTNode->bHighlightInSearchTree = bHighlight;
// 		}
// 	}
}

TSharedRef<SWidget> FFindInConversationResult::CreateIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = NULL;

	if (GraphNode.IsValid())
	{
// 		if (Cast<UConversationGraphNode_Service>(GraphNode.Get()))
// 		{
// 			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
// 		}
// 		else 
// 		if (Cast<UConversationGraphNode_Decorator>(GraphNode.Get()))
// 		{
// 			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.RefPinIcon"));
// 		}
//		else
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.FIB_Event"));
		}
	}
	
	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor);
}

FReply FFindInConversationResult::OnClick(TWeakPtr<class FConversationEditor> ConversationEditorPtr, TSharedPtr<FFindInConversationResult> Root)
{
	if (ConversationEditorPtr.IsValid() && GraphNode.IsValid())
	{
		if (Parent.IsValid() && Parent.HasSameObject(Root.Get()))
		{
			ConversationEditorPtr.Pin()->JumpToNode(GraphNode.Get());
		}
		else
		{
			ConversationEditorPtr.Pin()->JumpToNode(Parent.Pin()->GraphNode.Get());
		}
	}

	return FReply::Handled();
}

FString FFindInConversationResult::GetNodeTypeText() const
{
	if (GraphNode.IsValid())
	{
		FString NodeClassName = GraphNode->GetClass()->GetName();
		int32 Pos = NodeClassName.Find("_");
		if (Pos == INDEX_NONE)
		{
			return NodeClassName;
		}
		else
		{
			return NodeClassName.RightChop(Pos + 1);
		}
	}

	return FString();
}

FString FFindInConversationResult::GetCommentText() const
{
	if (GraphNode.IsValid())
	{
		return GraphNode->NodeComment;
	}

	return FString();
}

//////////////////////////////////////////////////////////////////////////
// SFindInConversation

void SFindInConversation::Construct( const FArguments& InArgs, TSharedPtr<FConversationEditor> InConversationEditor)
{
	ConversationEditorPtr = InConversationEditor;

	this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SAssignNew(SearchTextField, SSearchBox)
					.HintText(LOCTEXT("ConversationSearchHint", "Enter text to find nodes..."))
					.OnTextChanged(this, &SFindInConversation::OnSearchTextChanged)
					.OnTextCommitted(this, &SFindInConversation::OnSearchTextCommitted)
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SAssignNew(TreeView, STreeViewType)
					.TreeItemsSource(&ItemsFound)
					.OnGenerateRow(this, &SFindInConversation::OnGenerateRow)
					.OnGetChildren(this, &SFindInConversation::OnGetChildren)
					.OnSelectionChanged(this, &SFindInConversation::OnTreeSelectionChanged)
					.SelectionMode(ESelectionMode::Multi)
				]
			]
		];
}

void SFindInConversation::FocusForUse()
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
}

void SFindInConversation::OnSearchTextChanged(const FText& Text)
{
	SearchValue = Text.ToString();
	
	InitiateSearch();
}

void SFindInConversation::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	OnSearchTextChanged(Text);
}

void SFindInConversation::InitiateSearch()
{
	TArray<FString> Tokens;
	SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		(*It).Get()->SetNodeHighlight(false); // need to reset highlight
		TreeView->SetItemExpansion(*It, false);
	}
	ItemsFound.Empty();
	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchValue);
		MatchTokens(Tokens);
	}

	// Insert a fake result to inform user if none found
	if (ItemsFound.Num() == 0)
	{
		ItemsFound.Add(FSearchResult(new FFindInConversationResult(LOCTEXT("ConversationSearchNoResults", "No Results found").ToString())));
	}

	TreeView->RequestTreeRefresh();

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		TreeView->SetItemExpansion(*It, true);
	}
}

void SFindInConversation::MatchTokens(const TArray<FString>& Tokens)
{
	RootSearchResult.Reset();

	TWeakPtr<SGraphEditor> FocusedGraphEditor = ConversationEditorPtr.Pin()->GetFocusedGraphPtr();
	UEdGraph* Graph = NULL;
	if (FocusedGraphEditor.IsValid())
	{
		Graph = FocusedGraphEditor.Pin()->GetCurrentGraph();
	}

	if (Graph == NULL)
	{
		return;
	}

	RootSearchResult = FSearchResult(new FFindInConversationResult(FString("ConversationRoot")));

	for (auto It(Graph->Nodes.CreateConstIterator()); It; ++It)
	{
		UEdGraphNode* Node = *It;
			
		const FString NodeName = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();
		FSearchResult NodeResult(new FFindInConversationResult(NodeName, RootSearchResult, Node));

		FString NodeSearchString = NodeName + Node->GetClass()->GetName() + Node->NodeComment;
		NodeSearchString = NodeSearchString.Replace(TEXT(" "), TEXT(""));

		bool bNodeMatchesSearch = StringMatchesSearchTokens(Tokens, NodeSearchString);

		if (UConversationGraphNode* ConversationNode = Cast<UConversationGraphNode>(Node))
		{
			for (UAIGraphNode* SubNode : ConversationNode->SubNodes)
			{
				MatchTokensInChild(Tokens, SubNode, NodeResult);
			}
		}

		if ((NodeResult->Children.Num() > 0) || bNodeMatchesSearch)
		{
			NodeResult->SetNodeHighlight(true);
			ItemsFound.Add(NodeResult);
		}
	}
}

void SFindInConversation::MatchTokensInChild(const TArray<FString>& Tokens, UEdGraphNode* Child, FSearchResult ParentNode)
{
	if (Child == NULL)
	{
		return;
	}

	FString ChildName = Child->GetNodeTitle(ENodeTitleType::ListView).ToString();
	FString ChildSearchString = ChildName + Child->GetClass()->GetName() + Child->NodeComment;
	ChildSearchString = ChildSearchString.Replace(TEXT(" "), TEXT(""));
	if (StringMatchesSearchTokens(Tokens, ChildSearchString))
	{
		FSearchResult DecoratorResult(new FFindInConversationResult(ChildName, ParentNode, Child));
		ParentNode->Children.Add(DecoratorResult);
	}
}

TSharedRef<ITableRow> SFindInConversation::OnGenerateRow( FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(STableRow< TSharedPtr<FFindInConversationResult> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(450.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						InItem->CreateIcon()
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(2, 0)
					[
						SNew(STextBlock)
						.Text(FText::FromString(InItem->Value))
						.HighlightText(HighlightText)
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->GetNodeTypeText()))
				.HighlightText(HighlightText)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->GetCommentText()))
				.ColorAndOpacity(FLinearColor::Yellow)
				.HighlightText(HighlightText)
			]
		];
}

void SFindInConversation::OnGetChildren(FSearchResult InItem, TArray< FSearchResult >& OutChildren)
{
	OutChildren += InItem->Children;
}

void SFindInConversation::OnTreeSelectionChanged(FSearchResult Item , ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		Item->OnClick(ConversationEditorPtr, RootSearchResult);
	}
}

bool SFindInConversation::StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString)
{
	bool bFoundAllTokens = true;

	//search the entry for each token, it must have all of them to pass
	for (auto TokIT(Tokens.CreateConstIterator()); TokIT; ++TokIT)
	{
		const FString& Token = *TokIT;
		if (!ComparisonString.Contains(Token))
		{
			bFoundAllTokens = false;
			break;
		}
	}
	return bFoundAllTokens;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
