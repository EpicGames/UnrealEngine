// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
class UEdGraphNode;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename ItemType> class STreeView;

/** Item that matched the search results */
class FFindInConversationResult
{
public: 
	/** Create a root (or only text) result */
	FFindInConversationResult(const FString& InValue);
	
	/** Create a BT node result */
	FFindInConversationResult(const FString& InValue, TSharedPtr<FFindInConversationResult>& InParent, UEdGraphNode* InNode);

	/** Called when user clicks on the search item */
	FReply OnClick(TWeakPtr<class FConversationEditor> ConversationEditor,  TSharedPtr<FFindInConversationResult> Root);

	/** Create an icon to represent the result */
	TSharedRef<SWidget>	CreateIcon() const;

	/** Gets the comment on this node if any */
	FString GetCommentText() const;

	/** Gets the node type */
	FString GetNodeTypeText() const;

	/** Highlights conversation graph nodes */
	void SetNodeHighlight(bool bHighlight);

	/** Any children listed under this conversation node */
	TArray< TSharedPtr<FFindInConversationResult> > Children;

	/** The string value for this result */
	FString Value;

	/** The graph node that this search result refers to */
	TWeakObjectPtr<UEdGraphNode> GraphNode;

	/** Search result parent */
	TWeakPtr<FFindInConversationResult> Parent;
};

/** Widget for searching for (BT nodes) across focused Conversation */
class SFindInConversation : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFindInConversation){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class FConversationEditor> InConversationEditor);

	/** Focuses this widget's search box */
	void FocusForUse();

private:
	typedef TSharedPtr<FFindInConversationResult> FSearchResult;
	typedef STreeView<FSearchResult> STreeViewType;

	/** Called when user changes the text they are searching for */
	void OnSearchTextChanged(const FText& Text);

	/** Called when user commits text */
	void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/** Get the children of a row */
	void OnGetChildren(FSearchResult InItem, TArray<FSearchResult>& OutChildren);

	/** Called when user clicks on a new result */
	void OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type SelectInfo);

	/** Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Begins the search based on the SearchValue */
	void InitiateSearch();
	
	/** Find any results that contain all of the tokens */
	void MatchTokens(const TArray<FString>& Tokens);

	/** Find if child contains all of the tokens and add a result accordingly */
	void MatchTokensInChild(const TArray<FString>& Tokens, UEdGraphNode* Child, FSearchResult ParentNode);
	
	/** Determines if a string matches the search tokens */
	static bool StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString);

private:
	/** Pointer back to the behavior tree editor that owns us */
	TWeakPtr<class FConversationEditor> ConversationEditorPtr;
	
	/** The tree view displays the results */
	TSharedPtr<STreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<class SSearchBox> SearchTextField;
	
	/** This buffer stores the currently displayed results */
	TArray<FSearchResult> ItemsFound;

	/** we need to keep a handle on the root result, because it won't show up in the tree */
	FSearchResult RootSearchResult;

	/** The string to highlight in the results */
	FText HighlightText;

	/** The string to search for */
	FString	SearchValue;
};
