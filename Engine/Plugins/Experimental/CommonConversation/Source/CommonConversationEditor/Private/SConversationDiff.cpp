// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConversationDiff.h"
#include "DetailsViewArgs.h"
#include "SlateOptMacros.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SButton.h"
#include "IDetailsView.h"
#include "Widgets/Views/SListView.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "DiffResults.h"
#include "ConversationGraph.h"
#include "ConversationGraphNode.h"
#include "ConversationDatabase.h"
#include "PropertyEditorModule.h"
#include "GraphDiffControl.h"
#include "EdGraphUtilities.h"
#include "ConversationEditorUtils.h"
// #include "Conversation/Conversation.h"
#include "Framework/Commands/GenericCommands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ConversationCompiler.h"

#define LOCTEXT_NAMESPACE "SConversationDiff"

//////////////////////////////////////////////////////////////////////////
// FTreeDiffResultItem

struct FTreeDiffResultItem : public TSharedFromThis<FTreeDiffResultItem>
{
	/**
	 * Constructor
	 * @param InResult A difference result 
	 */
	FTreeDiffResultItem(const FDiffSingleResult& InResult): Result(InResult){}

	/**
	 * GenerateWidget for the diff item
	 * @return The Widget
	 */
	TSharedRef<SWidget>	GenerateWidget() const
	{
		FText ToolTip = Result.ToolTip;
		FLinearColor Color = Result.GetDisplayColor();
		FText Text = Result.DisplayString;
		if(Text.IsEmpty())
		{
			Text = LOCTEXT("DIF_UnknownDiff", "Unknown Diff");
			ToolTip = LOCTEXT("DIF_Confused", "There is an unspecified difference");
		}
		return SNew(STextBlock)
			.ToolTipText(ToolTip)
			.ColorAndOpacity(Color)
			.Text(Text);
	}

	// A result of a diff
	const FDiffSingleResult Result;
};


//////////////////////////////////////////////////////////////////////////
// FDiffListCommands

class FDiffListCommands : public TCommands<FDiffListCommands>
{
public:
	/** Constructor */
	FDiffListCommands() 
		: TCommands<FDiffListCommands>("DiffList", LOCTEXT("Diff", "Behavior Tree Diff"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Previous, "Prev", "Go to previous difference", EUserInterfaceActionType::Button, FInputChord(EKeys::F7, EModifierKey::Control));
		UI_COMMAND(Next, "Next", "Go to next difference", EUserInterfaceActionType::Button, FInputChord(EKeys::F7));
	}

	/** Go to previous difference */
	TSharedPtr<FUICommandInfo> Previous;

	/** Go to next difference */
	TSharedPtr<FUICommandInfo> Next;
};

//////////////////////////////////////////////////////////////////////////
// SConversationDiff

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SConversationDiff::Construct( const FArguments& InArgs )
{
	LastOtherPinTarget = nullptr;

	FDiffListCommands::Register();

	PanelOld.ConversationBank = InArgs._OldBank;
	PanelNew.ConversationBank = InArgs._NewBank;

	PanelOld.RevisionInfo = InArgs._OldRevision;
	PanelNew.RevisionInfo = InArgs._NewRevision;

	PanelOld.bShowAssetName = InArgs._ShowAssetNames;
	PanelNew.bShowAssetName = InArgs._ShowAssetNames;

	OpenInDefaults = InArgs._OpenInDefaults;

	TSharedRef<SHorizontalBox> DefaultEmptyPanel = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ConversationDiffGraphsToolTip", "Select Graph to Diff"))
		];

	this->ChildSlot
	[	
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Content()
		[
			SNew(SSplitter)
			+SSplitter::Slot()
			.Value(0.2f)
			[
				SNew(SBorder)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						//open in p4dif tool
						SNew(SButton)
						.OnClicked(this, &SConversationDiff::OnOpenInDefaults)
						.Content()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DiffConversationDefaults", "Default Diff"))
						]
					]
					+SVerticalBox::Slot()
					.FillHeight(1.f)
					[
						GenerateDiffListWidget()
					]
				]
			]
			+SSplitter::Slot()
			.Value(0.8f)
			[
				// Diff Window
				SNew(SSplitter)
				+SSplitter::Slot()
				.Value(0.5f)
				[
					// Left Diff
					SAssignNew(PanelOld.GraphEditorBorder, SBorder)
					.VAlign(VAlign_Fill)
					[
						DefaultEmptyPanel
					]
				]
				+SSplitter::Slot()
				.Value(0.5f)
				[
					// Right Diff
					SAssignNew(PanelNew.GraphEditorBorder, SBorder)
					.VAlign(VAlign_Fill)
					[
						DefaultEmptyPanel
					]
				]
			]
		]
	];

	//@TODO: CONVERSATION: Diff tool doesn't support more than one graph yet
	UEdGraph* OldGraph = PanelOld.ConversationBank? FConversationCompiler::GetGraphFromBank(PanelOld.ConversationBank, 0) : nullptr;
	UEdGraph* NewGraph = PanelNew.ConversationBank? FConversationCompiler::GetGraphFromBank(PanelNew.ConversationBank, 0) : nullptr;
	PanelOld.GeneratePanel(OldGraph, NewGraph);
	PanelNew.GeneratePanel(NewGraph, OldGraph);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SConversationDiff::OnOpenInDefaults()
{
	OpenInDefaults.ExecuteIfBound(PanelOld.ConversationBank, PanelNew.ConversationBank);
	return FReply::Handled();
}

TSharedRef<SWidget> SConversationDiff::GenerateDiffListWidget()
{
	BuildDiffSourceArray();
	if(DiffListSource.Num() > 0)
	{
		Algo::SortBy(DiffListSource, [](const FSharedDiffOnGraph& Data) { return Data->Result.Diff; });

		// Map commands through UI
		const FDiffListCommands& Commands = FDiffListCommands::Get();
		KeyCommands = MakeShareable(new FUICommandList );

		KeyCommands->MapAction(Commands.Previous, FExecuteAction::CreateSP(this, &SConversationDiff::PrevDiff));
		KeyCommands->MapAction(Commands.Next, FExecuteAction::CreateSP(this, &SConversationDiff::NextDiff));

		FToolBarBuilder ToolbarBuilder(KeyCommands.ToSharedRef(), FMultiBoxCustomization::None);
		ToolbarBuilder.AddToolBarButton(Commands.Previous, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff"));
		ToolbarBuilder.AddToolBarButton(Commands.Next, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff"));

		TSharedRef<SHorizontalBox> Result =	SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.MaxWidth(350.f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(0.f)
			.AutoHeight()
			[
				ToolbarBuilder.MakeWidget()
			]
			+SVerticalBox::Slot()
			.Padding(0.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("PropertyWindow.CategoryBackground"))
				.Padding(FMargin(2.0f))
				.ForegroundColor(FAppStyle::GetColor("PropertyWindow.CategoryForeground"))
				.ToolTipText(LOCTEXT("BehvaiorTreeDifDifferencesToolTip", "List of differences found between revisions, click to select"))
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RevisionDifferences", "Revision Differences"))
				]
			]
			+SVerticalBox::Slot()
			.Padding(1.f)
			.FillHeight(1.f)
			[
				SAssignNew(DiffList, SListViewType)
				.ListItemsSource(&DiffListSource)
				.OnGenerateRow(this, &SConversationDiff::OnGenerateRow)
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged(this, &SConversationDiff::OnSelectionChanged)
			]
		];
		return Result;
	}
	else
	{
		return SNew(SBorder).Visibility(EVisibility::Hidden);
	}
}

void SConversationDiff::BuildDiffSourceArray()
{
	TArray<FDiffSingleResult> FoundDiffs;
	//@TODO: CONVERSATION: Support diffing multiple graphs
	if (PanelOld.ConversationBank && PanelNew.ConversationBank)
	{
		FGraphDiffControl::DiffGraphs(FConversationCompiler::GetGraphFromBank(PanelOld.ConversationBank, 0), FConversationCompiler::GetGraphFromBank(PanelNew.ConversationBank, 0), FoundDiffs);
	}
	DiffListSource.Empty();
	for (auto DiffIt(FoundDiffs.CreateConstIterator()); DiffIt; ++DiffIt)
	{
		DiffListSource.Add(FSharedDiffOnGraph(new FTreeDiffResultItem(*DiffIt)));
	}
}

void SConversationDiff::NextDiff()
{
	int32 Index = (GetCurrentDiffIndex() + 1) % DiffListSource.Num();
	DiffList->SetSelection(DiffListSource[Index]);
}

void SConversationDiff::PrevDiff()
{
	int32 Index = GetCurrentDiffIndex();
	if(Index == 0)
	{
		Index = DiffListSource.Num() - 1;
	}
	else
	{
		Index = (Index - 1) % DiffListSource.Num();
	}
	DiffList->SetSelection(DiffListSource[Index]);
}

int32 SConversationDiff::GetCurrentDiffIndex()
{
	auto Selected = DiffList->GetSelectedItems();
	if(Selected.Num() == 1)
	{	
		int32 Index = 0;
		for(auto It(DiffListSource.CreateIterator());It;++It,Index++)
		{
			if(*It == Selected[0])
			{
				return Index;
			}
		}
	}
	return 0;
}

TSharedRef<ITableRow> SConversationDiff::OnGenerateRow(FSharedDiffOnGraph Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return	SNew(STableRow< FSharedDiffOnGraph >, OwnerTable)
			.Content()
			[
				Item->GenerateWidget()
			];
}

void SConversationDiff::OnSelectionChanged(FSharedDiffOnGraph Item, ESelectInfo::Type SelectionType)
{
	if(!Item.IsValid())
	{
		return;
	}

	//focus the graph onto the diff that was clicked on
	FDiffSingleResult Result = Item->Result;
	if(Result.Pin1)
	{
		PanelNew.GraphEditor.Pin()->ClearSelectionSet();
		PanelOld.GraphEditor.Pin()->ClearSelectionSet();

		auto FocusPin = [this](UEdGraphPin* InPin)
		{
			if (InPin)
			{
				UEdGraph* NodeGraph = InPin->GetOwningNode()->GetGraph();
				SGraphEditor* NodeGraphEditor = GetGraphEditorForGraph(NodeGraph);
				NodeGraphEditor->JumpToPin(InPin);
			}
		};

		FocusPin(Result.Pin1);
		FocusPin(Result.Pin2);
	}
	else if(Result.Node1)
	{
		PanelNew.GraphEditor.Pin()->ClearSelectionSet();
		PanelOld.GraphEditor.Pin()->ClearSelectionSet();

		auto FocusNode = [this](UEdGraphNode* InNode)
		{
			if (InNode)
			{
				UEdGraph* NodeGraph = InNode->GetGraph();
				SGraphEditor* NodeGraphEditor = GetGraphEditorForGraph(NodeGraph);

				UConversationGraphNode* BTNode = Cast<UConversationGraphNode>(InNode);
				if (BTNode && BTNode->bIsSubNode)
				{
					// This is a sub-node, we need to find our parent node in the graph
					// todo: work out why BTNode->ParentNode is always null
					TObjectPtr<UEdGraphNode>* ParentNodePtr = NodeGraph->Nodes.FindByPredicate([BTNode](UEdGraphNode* PotentialParentNode) -> bool
					{
						UConversationGraphNode* BTPotentialParentNode = Cast<UConversationGraphNode>(PotentialParentNode);
						return BTPotentialParentNode && (BTPotentialParentNode->SubNodes.Contains(BTNode));
					});

					// We need to call JumpToNode on the parent node, and then SetNodeSelection on the sub-node
					// as JumpToNode doesn't work for sub-nodes
					if (ParentNodePtr)
					{
						check(InNode->GetGraph() == (*ParentNodePtr)->GetGraph());
						NodeGraphEditor->JumpToNode(*ParentNodePtr, false, false);
					}
					NodeGraphEditor->SetNodeSelection(InNode, true);
				}
				else
				{
					NodeGraphEditor->JumpToNode(InNode, false);
				}
			}
		};

		FocusNode(Result.Node1);
		FocusNode(Result.Node2);
	}
}

SGraphEditor* SConversationDiff::GetGraphEditorForGraph(UEdGraph* Graph) const
{
	if(PanelOld.GraphEditor.Pin()->GetCurrentGraph() == Graph)
	{
		return PanelOld.GraphEditor.Pin().Get();
	}
	else if(PanelNew.GraphEditor.Pin()->GetCurrentGraph() == Graph)
	{
		return PanelNew.GraphEditor.Pin().Get();
	}
	checkNoEntry();
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FConversationDiffPanel
//////////////////////////////////////////////////////////////////////////

SConversationDiff::FConversationDiffPanel::FConversationDiffPanel()
{
	ConversationBank = nullptr;
}

void SConversationDiff::FConversationDiffPanel::GeneratePanel(UEdGraph* Graph, UEdGraph* GraphToDiff)
{
	TSharedPtr<SWidget> Widget = SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock).Text( LOCTEXT("BTDifPanelNoGraphTip", "Graph does not exist in this revision"))
		];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject(nullptr);
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateRaw(this, &SConversationDiff::FConversationDiffPanel::IsPropertyEditable));
	
	if(Graph)
	{
		SGraphEditor::FGraphEditorEvents InEvents;
		InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateRaw(this, &SConversationDiff::FConversationDiffPanel::OnSelectionChanged);

		FGraphAppearanceInfo AppearanceInfo;
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_BehaviorDif", "DIFF");

		if (!GraphEditorCommands.IsValid())
		{
			GraphEditorCommands = MakeShareable(new FUICommandList());

			GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
				FExecuteAction::CreateRaw(this, &FConversationDiffPanel::CopySelectedNodes),
				FCanExecuteAction::CreateRaw(this, &FConversationDiffPanel::CanCopyNodes));
		}

		auto Editor = SNew(SGraphEditor)
			.AdditionalCommands(GraphEditorCommands)
			.GraphToEdit(Graph)
			.GraphToDiff(GraphToDiff)
			.IsEditable(false)
			.TitleBar(SNew(SBorder).HAlign(HAlign_Center)
			[
				SNew(STextBlock).Text(GetTitle())
			])
			.Appearance(AppearanceInfo)
			.GraphEvents(InEvents);

		const FSlateBrush* ContentAreaBrush = FAppStyle::GetBrush( "Docking.Tab", ".ContentAreaBrush" );

		auto NewWidget = SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Value(0.8f)
			[
				Editor
			]
			+SSplitter::Slot()
			.Value(0.2f)
			[
				SNew( SBorder )
				.Visibility( EVisibility::Visible )
				.BorderImage( ContentAreaBrush )
				[
					DetailsView.ToSharedRef()
				]
			];

		GraphEditor = Editor;
		Widget = NewWidget;
	}

	GraphEditorBorder->SetContent(Widget.ToSharedRef());
}

FText SConversationDiff::FConversationDiffPanel::GetTitle() const
{
	FText Title = LOCTEXT("CurrentRevision", "Current Revision");

	// if this isn't the current working version being displayed
	if (!RevisionInfo.Revision.IsEmpty())
	{
		// Don't use grouping on the revision or CL numbers to match how Perforce displays them
		const FText DateText = FText::AsDate(RevisionInfo.Date, EDateTimeStyle::Short);
		const FText RevisionText = FText::FromString(RevisionInfo.Revision);
		const FText ChangelistText = FText::AsNumber(RevisionInfo.Changelist, &FNumberFormattingOptions::DefaultNoGrouping());

		if (bShowAssetName && ConversationBank)
		{
			FString AssetName = ConversationBank->GetName();
			if(ISourceControlModule::Get().GetProvider().UsesChangelists())
			{
				FText LocalizedFormat = LOCTEXT("NamedRevisionDiffFmtUsesChangelists", "{0} - Revision {1}, CL {2}, {3}");
				Title = FText::Format(LocalizedFormat, FText::FromString(AssetName), RevisionText, ChangelistText, DateText);
			}
			else
			{
				FText LocalizedFormat = LOCTEXT("NamedRevisionDiffFmt", "{0} - Revision {1}, {2}");
				Title = FText::Format(LocalizedFormat, FText::FromString(AssetName), RevisionText, DateText);
			}
		}
		else
		{
			if(ISourceControlModule::Get().GetProvider().UsesChangelists())
			{
				FText LocalizedFormat = LOCTEXT("PreviousRevisionDifFmtUsesChangelists", "Revision {0}, CL {1}, {2}");
				Title = FText::Format(LocalizedFormat, RevisionText, ChangelistText, DateText);
			}
			else
			{
				FText LocalizedFormat = LOCTEXT("PreviousRevisionDifFmt", "Revision {0}, {2}");
				Title = FText::Format(LocalizedFormat, RevisionText, DateText);
			}
		}
	}
	else if (bShowAssetName && ConversationBank)
	{
		FString AssetName = ConversationBank->GetName();
		FText LocalizedFormat = LOCTEXT("NamedCurrentRevisionFmt", "{0} - Current Revision");
		Title = FText::Format(LocalizedFormat, FText::FromString(AssetName));
	}

	return Title;
}

FGraphPanelSelectionSet SConversationDiff::FConversationDiffPanel::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin();
	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}
	return CurrentSelection;
}

void SConversationDiff::FConversationDiffPanel::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool SConversationDiff::FConversationDiffPanel::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != nullptr) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void SConversationDiff::FConversationDiffPanel::OnSelectionChanged( const FGraphPanelSelectionSet& NewSelection )
{
	ConversationEditorUtils::FPropertySelectionInfo SelectionInfo;
	TArray<UObject*> Selection = ConversationEditorUtils::GetSelectionForPropertyEditor(NewSelection, SelectionInfo);

	if (Selection.Num() == 1)
	{
		if (DetailsView.IsValid())
		{
			DetailsView->SetObjects(Selection);
		}
	}
	else if (DetailsView.IsValid())
	{
		DetailsView->SetObject(nullptr);
	}
}

bool SConversationDiff::FConversationDiffPanel::IsPropertyEditable()
{
	return false;
}

#undef LOCTEXT_NAMESPACE
