// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConversationTreeEditor.h"
#include "ConversationEditor.h"
#include "ConversationGraph.h"
#include "ConversationCompiler.h"
#include "ConversationGraphNode_EntryPoint.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "FConversationTreeNode"

//////////////////////////////////////////////////////////////////////////
// FConversationTreeNode

FConversationTreeNode::FConversationTreeNode(UEdGraphNode* InNode, const TSharedPtr<const FConversationTreeNode>& InParent)
	: ParentPtr(InParent)
	, GraphNodePtr(InNode)
{
}

TSharedRef<SWidget> FConversationTreeNode::CreateIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = nullptr;

	if (UEdGraphNode* GraphNode = GetGraphNode())
	{
		// 		if (Cast<UConversationGraphNode_Service>(GraphNode.Get()))
		// 		{
		// 			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
		// 		}
		// 		else 
		// 		if (Cast<UConversationGraphNode_Decorator>(GraphNode.Get()))
		// 		{
		// 			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.RefPinIcon"));
		// 		}
		//		else
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.FIB_Event"));
		}
	}

	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor);
}

FReply FConversationTreeNode::OnClick(TWeakPtr<FConversationEditor> ConversationEditorPtr)
{
	if (ConversationEditorPtr.IsValid())
	{
		if (UEdGraphNode* GraphNode = GetGraphNode())
		{
			ConversationEditorPtr.Pin()->JumpToNode(GraphNode);
		}
		//ConversationEditorPtr.Pin()->JumpToNode(Parent.Pin()->GraphNode.Get());
	}

	return FReply::Handled();
}

FText FConversationTreeNode::GetNodeTypeText() const
{
	if (UEdGraphNode* GraphNode = GetGraphNode())
	{
		return GraphNode->GetClass()->GetDisplayNameText();
	}

	return FText::GetEmpty();
}

FString FConversationTreeNode::GetCommentText() const
{
	if (UEdGraphNode* GraphNode = GetGraphNode())
	{
		return GraphNode->NodeComment;
	}

	return FString();
}

FText FConversationTreeNode::GetText() const
{
	if (UEdGraphNode* GraphNode = GetGraphNode())
	{
		return GraphNode->GetNodeTitle(ENodeTitleType::ListView);
	}

	return FText::GetEmpty();
}

const TArray< TSharedPtr<FConversationTreeNode> >& FConversationTreeNode::GetChildren() const
{
	if (bChildrenDirty)
	{
		bChildrenDirty = false;

		if (UConversationGraphNode* ConversationNode = Cast<UConversationGraphNode>(GetGraphNode()))
		{
			for (UAIGraphNode* SubNode : ConversationNode->SubNodes)
			{
				Children.Add(MakeShared<FConversationTreeNode>(SubNode, SharedThis(this)));
			}

			if (UEdGraphPin* OutputPin = ConversationNode->GetOutputPin())
			{
				for (UEdGraphPin* LinkedToPin : OutputPin->LinkedTo)
				{
					Children.Add(MakeShared<FConversationTreeNode>(LinkedToPin->GetOwningNode(), SharedThis(this)));
				}
			}
		}
	}

	return Children;
}

//////////////////////////////////////////////////////////////////////////
// SConversationTreeEditor

void SConversationTreeEditor::Construct( const FArguments& InArgs, TSharedPtr<FConversationEditor> InConversationEditor)
{
	ConversationEditorPtr = InConversationEditor;
	InConversationEditor->FocusedGraphEditorChanged.AddSP(this, &SConversationTreeEditor::OnFocusedGraphChanged);
	UConversationDatabase* ConversationAsset = InConversationEditor->GetConversationAsset();
	if (UConversationGraph* MyGraph = FConversationCompiler::GetGraphFromBank(ConversationAsset, 0))
	{
		MyGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &SConversationTreeEditor::OnGraphChanged));
	}

	BuildTree();

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(TreeView, STreeViewType)
				.TreeItemsSource(&RootNodes)
				.OnGenerateRow(this, &SConversationTreeEditor::OnGenerateRow)
				.OnGetChildren(this, &SConversationTreeEditor::OnGetChildren)
				.OnSelectionChanged(this, &SConversationTreeEditor::OnTreeSelectionChanged)
				.SelectionMode(ESelectionMode::Multi)
			]
		]
	];
}

void SConversationTreeEditor::OnFocusedGraphChanged()
{
	RefreshTree();
}

void SConversationTreeEditor::OnGraphChanged(const FEdGraphEditAction& Action)
{
	RefreshTree();
}

void SConversationTreeEditor::RefreshTree()
{
	BuildTree();

	TreeView->RequestTreeRefresh();

	for (auto It(RootNodes.CreateIterator()); It; ++It)
	{
		TreeView->SetItemExpansion(*It, true);
	}
}

void SConversationTreeEditor::BuildTree()
{
	RootNodes.Empty();

	TWeakPtr<SGraphEditor> FocusedGraphEditor = ConversationEditorPtr.Pin()->GetFocusedGraphPtr();
	UEdGraph* Graph = nullptr;
	if (FocusedGraphEditor.IsValid())
	{
		Graph = FocusedGraphEditor.Pin()->GetCurrentGraph();
	}

	if (Graph == nullptr)
	{
		return;
	}

	for (UEdGraphNode* Node : Graph->Nodes)
	{
		if (UConversationGraphNode_EntryPoint* RootNode = Cast<UConversationGraphNode_EntryPoint>(Node))
		{
			RootNodes.Add(MakeShared<FConversationTreeNode>(RootNode, TSharedPtr<FConversationTreeNode>()));
		}
	}
}

TSharedRef<ITableRow> SConversationTreeEditor::OnGenerateRow(TSharedPtr<FConversationTreeNode> InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(STableRow< TSharedPtr<FConversationTreeNode> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(450.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						InItem->CreateIcon()
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(2, 0)
					[
						SNew(STextBlock)
						.Text(InItem->GetText())
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InItem->GetNodeTypeText())
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->GetCommentText()))
				.ColorAndOpacity(FLinearColor::Yellow)
			]
		];
}

void SConversationTreeEditor::OnGetChildren(TSharedPtr<FConversationTreeNode> InItem, TArray< TSharedPtr<FConversationTreeNode> >& OutChildren)
{
	OutChildren.Append(InItem->GetChildren());
}

void SConversationTreeEditor::OnTreeSelectionChanged(TSharedPtr<FConversationTreeNode> Item, ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		Item->OnClick(ConversationEditorPtr);
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
