// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintUtilities.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
class UEdGraphNode;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class STreeView;

class FConversationEditor;

/** Item that matched the search results */
class FConversationTreeNode : public TSharedFromThis<FConversationTreeNode>
{
public:
	/** Create a BT node result */
	FConversationTreeNode(UEdGraphNode* InNode, const TSharedPtr<const FConversationTreeNode>& InParent);

	/** Called when user clicks on the search item */
	FReply OnClick(TWeakPtr<FConversationEditor> ConversationEditor);

	/** Create an icon to represent the result */
	TSharedRef<SWidget>	CreateIcon() const;

	/** Gets the comment on this node if any */
	FString GetCommentText() const;

	/** Gets the node type */
	FText GetNodeTypeText() const;

	FText GetText() const;

	UEdGraphNode* GetGraphNode() const { return GraphNodePtr.Get(); }

	const TArray< TSharedPtr<FConversationTreeNode> >& GetChildren() const;

	/** Search result parent */
	TWeakPtr<const FConversationTreeNode> ParentPtr;

private:
	mutable bool bChildrenDirty = true;

	/** Any children listed under this conversation node */
	mutable TArray< TSharedPtr<FConversationTreeNode> > Children;

	/** The graph node that this search result refers to */
	TWeakObjectPtr<UEdGraphNode> GraphNodePtr;
};

/** */
class SConversationTreeEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConversationTreeEditor){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class FConversationEditor> InConversationEditor);

private:
	typedef STreeView<TSharedPtr<FConversationTreeNode>> STreeViewType;

	/** Get the children of a row */
	void OnGetChildren(TSharedPtr<FConversationTreeNode> InItem, TArray<TSharedPtr<FConversationTreeNode>>& OutChildren);

	/** Called when user clicks on a new result */
	void OnTreeSelectionChanged(TSharedPtr<FConversationTreeNode> Item, ESelectInfo::Type SelectInfo);

	/** Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FConversationTreeNode> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void OnFocusedGraphChanged();
	void OnGraphChanged(const FEdGraphEditAction& Action);

	/** Begins the search based on the SearchValue */
	void RefreshTree();
	void BuildTree();
	
private:
	/** Pointer back to the behavior tree editor that owns us */
	TWeakPtr<class FConversationEditor> ConversationEditorPtr;
	
	/** The tree view displays the results */
	TSharedPtr<STreeViewType> TreeView;
	
	/** This buffer stores the currently displayed results */
	TArray<TSharedPtr<FConversationTreeNode>> RootNodes;

	/** The string to search for */
	FString	SearchValue;
};
