// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

#define UE_API COMMONCONVERSATIONEDITOR_API

class UConversationDatabase;

class FAssetTypeActions_ConversationDatabase : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ConversationDatabase", "Conversation Bank"); }
	virtual FColor GetTypeColor() const override { return FColor(149,70,255); }
	UE_API virtual UClass* GetSupportedClass() const override;
	UE_API virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override;
	UE_API virtual uint32 GetCategories() override;
	UE_API virtual void PerformAssetDiff(UObject* Asset1, UObject* Asset2, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override;

private:

	/* Called to open the Behavior Tree defaults view, this opens whatever text diff tool the user has */
	UE_API void OpenInDefaults(UConversationDatabase* OldBank, UConversationDatabase* NewBank) const;
};

#undef UE_API
