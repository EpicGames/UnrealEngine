// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FAssetTypeActions_Base;
struct FGraphPanelNodeFactory;

class FCommonConversationEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TArray<TSharedPtr<FAssetTypeActions_Base>> ItemDataAssetTypeActions;
};
