// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonConversationGraph : ModuleRules
{
	public CommonConversationGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"CommonConversationRuntime",
				"GraphEditor",
				"AIGraph",
				"AIModule",
				"UnrealEd",
				"KismetWidgets",
				"ToolMenus",
				"PropertyEditor",
				"GameplayTags"
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
