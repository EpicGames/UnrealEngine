// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonConversationGraphModule.h"

#include "ConversationDatabase.h"
#include "ConversationCompiler.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/CoreRedirects.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY(LogCommonConversationGraph);

#define LOCTEXT_NAMESPACE "CommonConversationEditor"

void FCommonConversationGraphModule::StartupModule()
{
	UPackage::PreSavePackageWithContextEvent.AddRaw(this, &FCommonConversationGraphModule::HandlePreSavePackage);

#if WITH_EDITOR
	FEditorDelegates::BeginPIE.AddRaw(this, &FCommonConversationGraphModule::HandleBeginPIE);
#endif

	TArray<FCoreRedirect> Redirects;
	Redirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/Script/CommonDialogueGraph"), TEXT("/Script/CommonConversationGraph"));
	FCoreRedirects::AddRedirectList(Redirects, TEXT("CommonConversationGraph"));
}

void FCommonConversationGraphModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	UPackage::PreSavePackageWithContextEvent.RemoveAll(this);

#if WITH_EDITOR
	FEditorDelegates::BeginPIE.RemoveAll(this);
#endif
}

void FCommonConversationGraphModule::HandlePreSavePackage(UPackage* Package, FObjectPreSaveContext ObjectSaveContext)
{
	TArray<UObject*> Objects;
	GetObjectsWithPackage(Package, Objects, EGetObjectsFlags::None);
	for (UObject* RootPackageObject : Objects)
	{
		if (UConversationDatabase* Database = Cast<UConversationDatabase>(RootPackageObject))
		{
			FConversationCompiler::RebuildBank(Database);
		}
	}
}

void FCommonConversationGraphModule::HandleBeginPIE(bool bIsSimulating)
{
	FConversationCompiler::ScanAndRecompileOutOfDateCompiledConversations();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCommonConversationGraphModule, CommonConversationGraph)
