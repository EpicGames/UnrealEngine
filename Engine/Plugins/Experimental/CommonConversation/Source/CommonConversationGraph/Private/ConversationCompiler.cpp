// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationCompiler.h"

#include "CommonConversationGraphModule.h"
#include "ConversationGraph.h"
#include "ConversationGraphSchema.h"
#include "ConversationDatabase.h"


#include "ConversationSubNode.h"
#include "ConversationGraphNode_Task.h"

#include "ConversationEntryPointNode.h"
#include "ConversationGraphNode_EntryPoint.h"
#include "ConversationGraphNode_Knot.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "Stats/StatsMisc.h"
#include "ConversationLinkNode.h"
#include "Engine/AssetManager.h"
#include "Misc/MessageDialog.h"
#include "FileHelpers.h"

//////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "FConversationCompiler"

enum class EConversationCompilerVersion
{
	Initial,
	AddedSupportForLazyloadMetadata,
	ChangedTheEntryTagsStructureToIncludeGuid,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

namespace UE::CommonConversationGraph::Private
{
	static int32 GetNodeXLocation(const UEdGraphPin* Pin)
	{
		check(Pin);
		const UEdGraphNode* Node = Pin->GetOwningNode();
		check(Node);
		return Node->NodePosX;
	}
}

//@TODO: CONVERSATION: Push this into the engine, and update UAssetToolsImpl::CreateUniqueAssetName, FBlueprintEditorUtils::FindUniqueKismetName, FNiagaraUtilities::GetUniqueName, etc... to use it
struct FUniqueNameGenerator
{
private:
	FString TrimmedBaseName;
	int32 IntSuffix = 0;
	int32 TrailingIntegerLength = 0;

public:
	FUniqueNameGenerator(const FString& InSeedName)
	{
		int32 CharIndex = InSeedName.Len() - 1;
		while (CharIndex >= 0 && InSeedName[CharIndex] >= TEXT('0') && InSeedName[CharIndex] <= TEXT('9'))
		{
			--CharIndex;
		}

		if ((InSeedName.Len() > 0) && (CharIndex == -1))
		{
			// This is the all numeric name, in this case we'd like to append _number, because just adding a number isn't great
			TrimmedBaseName = InSeedName + TEXT("_");
			IntSuffix = 2;
		}
		else if ((CharIndex >= 0) && (CharIndex < InSeedName.Len() - 1))
		{
			const FString TrailingInteger = InSeedName.RightChop(CharIndex + 1);
			TrailingIntegerLength = TrailingInteger.Len();
			TrimmedBaseName = InSeedName.Left(CharIndex + 1);
			IntSuffix = FCString::Atoi(*TrailingInteger);
		}
		else
		{
			TrimmedBaseName = InSeedName;
		}
	}

	// Generates the next name (does not test for uniqueness, that's up to the caller)
	FString GenerateName()
	{
		FString Result;
		if (IntSuffix < 1)
		{
			Result = TrimmedBaseName;
		}
		else
		{
			FString Suffix = FString::Printf(TEXT("%d"), IntSuffix);
			while (Suffix.Len() < TrailingIntegerLength)
			{
				Suffix = TEXT("0") + Suffix;
			}
			Result = FString::Printf(TEXT("%s%s"), *TrimmedBaseName, *Suffix);
		}

		IntSuffix++;

		return Result;
	}

	// Generates a name that will be unique within the specified outer
	FName GenerateUniqueNameWithinOuter(UObject* Outer)
	{
		while (true)
		{
			FName TestName(*GenerateName());
			if (FindObjectWithOuter(Outer, nullptr, TestName) == nullptr)
			{
				return TestName;
			}
		}
	}
};

//////////////////////////////////////////////////////////////////////
// FConversationCompiler


/** Struct that contains all important identifier that define an entry point. */
struct FConversationEntryInfo
{
	FConversationEntryInfo(FGameplayTag InEntryTag, FString InEntryIdentifier) 
	{
		EntryTag = InEntryTag;
		EntryIdentifier = InEntryIdentifier;
	}

	bool operator==(const FConversationEntryInfo& Other) const
	{
		return EntryTag == Other.EntryTag && EntryIdentifier == Other.EntryIdentifier;
	}

	bool operator!=(const FConversationEntryInfo& Other) const
	{
		return EntryTag != Other.EntryTag || EntryIdentifier != Other.EntryIdentifier;
	}

	FGameplayTag EntryTag;
	FString EntryIdentifier;
};

/** Used so we can have a TMap of this struct */
FORCEINLINE uint32 GetTypeHash(const FConversationEntryInfo& EntryIdentifiersStruct)
{
	return HashCombineFast(GetTypeHash(EntryIdentifiersStruct.EntryTag), GetTypeHash(EntryIdentifiersStruct.EntryIdentifier));
}


int32 FConversationCompiler::GetCompilerVersion()
{
	return (int32)EConversationCompilerVersion::LatestVersion;
}

UConversationGraph* FConversationCompiler::CreateNewGraph(UConversationDatabase* ConversationAsset, FName GraphName)
{
	UConversationGraph* NewGraph = CastChecked<UConversationGraph>(FBlueprintEditorUtils::CreateNewGraph(ConversationAsset, GraphName, UConversationGraph::StaticClass(), UConversationGraphSchema::StaticClass()));

	const UEdGraphSchema* Schema = NewGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*NewGraph);

	NewGraph->OnCreated();

	return NewGraph;
}

UConversationGraph* FConversationCompiler::AddNewGraph(UConversationDatabase* ConversationAsset, const FString& DesiredName)
{
	// Find a unique default name for the duplicated asset
	FUniqueNameGenerator NameGenerator(DesiredName);
	const FName GraphName = NameGenerator.GenerateUniqueNameWithinOuter(ConversationAsset);

	check(ConversationAsset);
	UConversationGraph* NewGraph = CreateNewGraph(ConversationAsset, GraphName);
	ConversationAsset->SourceGraphs.Add(NewGraph);
	return NewGraph;
}

int32 FConversationCompiler::GetNumGraphs(UConversationDatabase* ConversationAsset)
{
	check(ConversationAsset);
	return ConversationAsset->SourceGraphs.Num();
}

UConversationGraph* FConversationCompiler::GetGraphFromBank(UConversationDatabase* ConversationAsset, int32 Index)
{
	check(ConversationAsset);
	return ConversationAsset->SourceGraphs.IsValidIndex(Index) ? CastChecked<UConversationGraph>(ConversationAsset->SourceGraphs[Index]) : nullptr;
}

void FConversationCompiler::RebuildBank(UConversationDatabase* ConversationAsset)
{
	SCOPE_LOG_TIME_IN_SECONDS(TEXT("FConversationCompiler::RebuildBank"), nullptr);

	check(ConversationAsset);

	ConversationAsset->CompilerVersion = GetCompilerVersion();

	// Merge all the graphs
	TArray<UConversationGraphNode*> AllGraphNodes;
	for (UEdGraph* Graph : ConversationAsset->SourceGraphs)
	{
		Graph->GetNodesOfClass<UConversationGraphNode>(/*inout*/ AllGraphNodes);
	}

	// Clear all error messages and add to the full nodes map (used for the editor only)
	ConversationAsset->FullNodeMap.Reset();
	for (UConversationGraphNode* EdNode : AllGraphNodes)
	{
		EdNode->ErrorMsg.Reset();

		if (EdNode->NodeInstance == nullptr)
		{
			EdNode->ErrorMsg = TEXT("Unknown Node");
			continue;
		}

		check(EdNode->GetRuntimeNode<UConversationNode>());
		check(EdNode->NodeGuid.IsValid());

		// Add to the editor full nodes map
		if (ConversationAsset->FullNodeMap.Contains(EdNode->NodeGuid))
		{
			EdNode->ErrorMsg = TEXT("Duplicate GUID");
		}
		else
		{
			ConversationAsset->FullNodeMap.Add(EdNode->NodeGuid, EdNode->GetRuntimeNode<UConversationNode>());
		}

		// Wire up subnodes
		if (UConversationGraphNode_Task* EdTaskNode = Cast<UConversationGraphNode_Task>(EdNode))
		{
			UConversationTaskNode* TaskNode = EdTaskNode->GetRuntimeNode<UConversationTaskNode>();
			TaskNode->SubNodes.Reset();
			for (UAIGraphNode* SubNode : EdTaskNode->SubNodes)
			{
				UConversationGraphNode* TypedSubNode = Cast<UConversationGraphNode>(SubNode);
				if (ensure(TypedSubNode))
				{
					UConversationSubNode* TaskSubNode = TypedSubNode->GetRuntimeNode<UConversationSubNode>();
					if (ensure(TaskSubNode))
					{
						TaskNode->SubNodes.Add(TaskSubNode);
					}
					else
					{
						UE_LOGF(LogCommonConversationGraph, Warning, "Invalid TaskSubNode inside Conversation Asset %ls", *ConversationAsset->GetPrimaryAssetId().ToString());
					}
				}
				else
				{
					UE_LOGF(LogCommonConversationGraph, Warning, "Invalid TypedSubNode inside Conversation Asset %ls", *ConversationAsset->GetPrimaryAssetId().ToString());
				}
			}
		}
		
		// Wire up links
		if (UConversationNodeWithLinks* RuntimeNodeWithLinks = Cast<UConversationNodeWithLinks>(EdNode->NodeInstance))
		{
			RuntimeNodeWithLinks->OutputConnections.Reset();

			UEdGraphPin* OutputPin = EdNode->GetOutputPin();
			check(OutputPin);

			ForeachConnectedOutgoingConversationNodeSorted(OutputPin, [RuntimeNodeWithLinks](UConversationGraphNode* RemoteNode)
			{
				RuntimeNodeWithLinks->OutputConnections.Add(RemoteNode->NodeGuid);
				if (UConversationNodeWithLinks* ChildNodeWithLinks = Cast<UConversationNodeWithLinks>(RemoteNode->NodeInstance))
				{
					// Tell child about its parent so calls to 'GetParentNode' work as expected
					// Could alternatively pass as FGuid and store similarly to 'output connections' (from RuntimeNodeWithLinks->GetNodeGuid()), 
					// Users would need to call UConversationRegistry::GetRuntimeNodeFromGUID when they want the parent node in that case
					ChildNodeWithLinks->InitializeNode(RuntimeNodeWithLinks);
				}
			});
		}
	}

	// Gather all entry points
	ConversationAsset->EntryTags.Reset();
	TMap<FConversationEntryInfo, TArray<FGuid>> EntryMap;
	TArray<UConversationGraphNode_EntryPoint*> EntryGraphNodes;
	for (UConversationGraphNode* EdNode : AllGraphNodes)
	{
		if (UConversationGraphNode_EntryPoint* EdEntryNode = Cast<UConversationGraphNode_EntryPoint>(EdNode))
		{
			UConversationEntryPointNode* EntryNode = EdEntryNode->GetRuntimeNode<UConversationEntryPointNode>();
			if (ensure(EntryNode))
			{
				if (EntryNode->EntryTag.IsValid())
				{
					EntryMap.FindOrAdd(FConversationEntryInfo(EntryNode->EntryTag, EntryNode->GetIdentifier())).Add(EdEntryNode->NodeGuid);
					EntryGraphNodes.Add(EdEntryNode);
				}
				else
				{
					EdEntryNode->ErrorMsg = TEXT("No EntryTag set");
				}
			}
		}
	}

	// Add the resulting entry nodes to the entry list.
	for (const auto& KVP : EntryMap)
	{
		FConversationEntryList Entry;
		Entry.EntryTag = KVP.Key.EntryTag;
		Entry.EntryIdentifier = KVP.Key.EntryIdentifier;
		Entry.DestinationList.Append(KVP.Value);

		ConversationAsset->EntryTags.Add(Entry);
	}

	// Determine reachability of the rest of the nodes
	TSet<UConversationGraphNode*> ReachableNodeSet;
	{
		TArray<UConversationGraphNode*> ReachableStack;
		ReachableStack.Append(EntryGraphNodes);

		while (ReachableStack.Num() > 0)
		{
			UConversationGraphNode* Candidate = ReachableStack.Pop();

			if (!ReachableNodeSet.Contains(Candidate) && !Candidate->HasErrors())
			{
				ReachableNodeSet.Add(Candidate);

				if (UEdGraphPin* OutputPin = Candidate->GetOutputPin())
				{
					ForeachConnectedOutgoingConversationNode(OutputPin, [&ReachableStack](UConversationGraphNode* RemoteNode) {
						ReachableStack.Add(RemoteNode);
					});
				}
			}
		}
	}

	// Add the reachable nodes to the map
	ConversationAsset->ReachableNodeMap.Reset();
	ConversationAsset->InternalNodeIds.Reset();
	ConversationAsset->ExitTags.Reset();
	for (UConversationGraphNode* EdNode : ReachableNodeSet)
	{
		UConversationNode* NodeInstance = CastChecked<UConversationNode>(EdNode->NodeInstance);
		NodeInstance->Compiled_NodeGUID = EdNode->NodeGuid;

		// (no need to check for uniqueness, we already did that for all nodes above)
		ConversationAsset->ReachableNodeMap.Add(EdNode->NodeGuid, NodeInstance);
		ConversationAsset->InternalNodeIds.Add(EdNode->NodeGuid);

		if (UConversationLinkNode* ExitTagNode = Cast<UConversationLinkNode>(NodeInstance))
		{
			ConversationAsset->ExitTags.AddTag(ExitTagNode->GetRemoteEntryTag());
		}
	}

// 	TMap<FGameplayTag, UConversationNode*> EntryMap;
// 	TMap<FGuid, UConversationNode*> NodeMap;
// 	TArray<FCommonDialogueBankParticipant> Speakers;
// 	TMap<FGuid, UConversationNode*> FullNodeMap;


	//@TODO: CONVERSATION: Do stuff here for runtime use
	// See UBehaviorTreeComponent::RequestExecution


	//UPROPERTY(AssetRegistrySearchable)
	//TArray<FGuid> LinkedToNodeIds;

}

void FConversationCompiler::ForeachConnectedOutgoingConversationNode(UEdGraphPin* Pin, TFunctionRef<void(UConversationGraphNode*)> Predicate)
{
	for (UEdGraphPin* RemotePin : Pin->LinkedTo)
	{
		if (UConversationGraphNode_Knot* Knot = Cast<UConversationGraphNode_Knot>(RemotePin->GetOwningNode()))
		{
			ForeachConnectedOutgoingConversationNode(Knot->GetOutputPin(), Predicate);
		}
		else if (UConversationGraphNode* RemoteNode = Cast<UConversationGraphNode>(RemotePin->GetOwningNode()))
		{
			Predicate(RemoteNode);
		}
	}
}

void FConversationCompiler::ForeachConnectedOutgoingConversationNodeSorted(UEdGraphPin* Pin, TFunctionRef<void(UConversationGraphNode*)> Predicate)
{
	TArray<UEdGraphPin*> SortedPins = Pin->LinkedTo;
	Algo::SortBy(SortedPins, UE::CommonConversationGraph::Private::GetNodeXLocation);

	for (UEdGraphPin* RemotePin : SortedPins)
	{
		if (UConversationGraphNode_Knot* Knot = Cast<UConversationGraphNode_Knot>(RemotePin->GetOwningNode()))
		{
			ForeachConnectedOutgoingConversationNode(Knot->GetOutputPin(), Predicate);
		}
		else if (UConversationGraphNode* RemoteNode = Cast<UConversationGraphNode>(RemotePin->GetOwningNode()))
		{
			Predicate(RemoteNode);
		}
	}
}

void FConversationCompiler::ScanAndRecompileOutOfDateCompiledConversations()
{
	TArray<FAssetData> AllConversations;
	UAssetManager::Get().GetPrimaryAssetDataList(FPrimaryAssetType(UConversationDatabase::StaticClass()->GetFName()), AllConversations);

	TArray<UPackage*> OutOfDateConversationPackages;

	for (FAssetData& ConversationAsset : AllConversations)
	{
		const int32 ConversationCompilerVersion = ConversationAsset.GetTagValueRef<int32>(GET_MEMBER_NAME_CHECKED(UConversationDatabase, CompilerVersion));
		if (ConversationCompilerVersion < GetCompilerVersion())
		{
			if (UConversationDatabase* ConversationDB = Cast<UConversationDatabase>(ConversationAsset.GetAsset()))
			{
				FConversationCompiler::RebuildBank(ConversationDB);
				OutOfDateConversationPackages.Add(ConversationDB->GetOutermost());
			}
		}
	}

	if (OutOfDateConversationPackages.Num() > 0)
	{
		EAppReturnType::Type SaveConversations = FMessageDialog::Open(EAppMsgType::YesNo,
			FText::Format(LOCTEXT("ResaveConversations", "We found {0} conversations on an old version of the compiler that need to be resaved.\n\nSave?"), OutOfDateConversationPackages.Num())
		);

		if (SaveConversations == EAppReturnType::Yes)
		{
			FEditorFileUtils::PromptForCheckoutAndSave(OutOfDateConversationPackages, /*bCheckDirty*/false, /*bPromptToSave*/false);
		}
	}
}

#undef LOCTEXT_NAMESPACE
