// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraph.h"
#include "ConversationGraphSchema.h"
#include "ConversationDatabase.h"
#include "AIGraphNode.h"
#include "ConversationCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraph)

//////////////////////////////////////////////////////////////////////
// UConversationGraph

UConversationGraph::UConversationGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Schema = UConversationGraphSchema::StaticClass();
	bLockUpdates = false;
}

void UConversationGraph::UpdateAsset(int32 UpdateFlags)
{
	if (bLockUpdates)
	{
		return;
	}

	// Fix up the parent node pointers (which are marked transient for some reason)
	for (UEdGraphNode* Node : Nodes)
	{
		if (UAIGraphNode* AINode = Cast<UAIGraphNode>(Node))
		{
			for (UAIGraphNode* SubNode : AINode->SubNodes)
			{
				if (SubNode != nullptr)
				{
					SubNode->ParentNode = AINode;
				}
			}
		}
	}

	UConversationDatabase* ConversationAsset = CastChecked<UConversationDatabase>(GetOuter());
	FConversationCompiler::RebuildBank(ConversationAsset);
}

