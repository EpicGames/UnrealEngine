// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphConnectionDrawingPolicy.h"
#include "ConversationEditorColors.h"
#include "ConversationGraphNode.h"
#include "SGraphNode.h"

FConversationGraphConnectionDrawingPolicy::FConversationGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(InGraphObj)
{
}

void FConversationGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	Params.WireThickness = 1.5f;

	Params.WireColor = ConversationEditorColors::Connection::Default;

	UConversationGraphNode* FromNode = OutputPin ? Cast<UConversationGraphNode>(OutputPin->GetOwningNode()) : NULL;
	UConversationGraphNode* ToNode = InputPin ? Cast<UConversationGraphNode>(InputPin->GetOwningNode()) : NULL;
	if (ToNode && FromNode)
	{
#ifdef TODO_CONVERSATION_EDITOR //@TODO: CONVERSATION: Find uses of TODO_CONVERSATION_EDITOR
		if ((ToNode->bDebuggerMarkCurrentlyActive && FromNode->bDebuggerMarkCurrentlyActive) ||
			(ToNode->bDebuggerMarkPreviouslyActive && FromNode->bDebuggerMarkPreviouslyActive))
		{
			Params.WireThickness = 10.0f;
			Params.bDrawBubbles = true;
		}
		else if (FConversationDebugger::IsPlaySessionPaused())
		{
			UConversationGraphNode* FirstToNode = ToNode;
			int32 FirstPathIdx = ToNode->DebuggerSearchPathIndex;
			for (int32 i = 0; i < ToNode->Decorators.Num(); i++)
			{
				UConversationGraphNode* TestNode = ToNode->Decorators[i];
				if (TestNode->DebuggerSearchPathIndex != INDEX_NONE &&
					(TestNode->bDebuggerMarkSearchSucceeded || TestNode->bDebuggerMarkSearchFailed))
				{
					if (TestNode->DebuggerSearchPathIndex < FirstPathIdx || FirstPathIdx == INDEX_NONE)
					{
						FirstPathIdx = TestNode->DebuggerSearchPathIndex;
						FirstToNode = TestNode;
					}
				}
			}

			if (FirstToNode->bDebuggerMarkSearchSucceeded || FirstToNode->bDebuggerMarkSearchFailed)
			{
				Params.WireThickness = 5.0f;
				Params.WireColor = FirstToNode->bDebuggerMarkSearchSucceeded ? ConversationEditorColors::Debugger::SearchSucceeded :
					ConversationEditorColors::Debugger::SearchFailed;

				// Use the bUserFlag1 flag to indicate that we need to reverse the direction of connection (used by debugger)
				Params.bUserFlag1 = true;
			}
		}
#endif
	}

	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
	if (bDeemphasizeUnhoveredPins)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}
}

void FConversationGraphConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	// Build an acceleration structure to quickly find geometry for the nodes
	NodeWidgetMap.Empty();
	for (int32 NodeIndex = 0; NodeIndex < ArrangedNodes.Num(); ++NodeIndex)
	{
		FArrangedWidget& CurWidget = ArrangedNodes[NodeIndex];
		TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(CurWidget.Widget);
		NodeWidgetMap.Add(ChildNode->GetNodeObj(), NodeIndex);
	}

	// Now draw
	FConnectionDrawingPolicy::Draw(InPinGeometries, ArrangedNodes);
}

void FConversationGraphConnectionDrawingPolicy::DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin)
{
	FConnectionParams Params;
	DetermineWiringStyle(Pin, nullptr, /*inout*/ Params);

	if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		DrawSplineWithArrow(FGeometryHelper::FindClosestPointOnGeom(PinGeometry, EndPoint), EndPoint, Params);
	}
	else
	{
		DrawSplineWithArrow(FGeometryHelper::FindClosestPointOnGeom(PinGeometry, StartPoint), StartPoint, Params);
	}
}

void FConversationGraphConnectionDrawingPolicy::DrawSplineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params)
{
	// bUserFlag1 indicates that we need to reverse the direction of connection (used by debugger)
	const FVector2f& P0 = Params.bUserFlag1 ? EndAnchorPoint : StartAnchorPoint;
	const FVector2f& P1 = Params.bUserFlag1 ? StartAnchorPoint : EndAnchorPoint;

	Internal_DrawLineWithArrow(P0, P1, Params);
}

void FConversationGraphConnectionDrawingPolicy::Internal_DrawLineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params)
{
	//@TODO: Should this be scaled by zoom factor?
	const float LineSeparationAmount = 4.5f;

	const FVector2f DeltaPos = EndAnchorPoint - StartAnchorPoint;
	const FVector2f UnitDelta = DeltaPos.GetSafeNormal();
	const FVector2f Normal = FVector2f(DeltaPos.Y, -DeltaPos.X).GetSafeNormal();

	// Come up with the final start/end points
	const FVector2f DirectionBias = Normal * LineSeparationAmount;
	const FVector2f LengthBias = ArrowRadius.X * UnitDelta;
	const FVector2f StartPoint = StartAnchorPoint + DirectionBias + LengthBias;
	const FVector2f EndPoint = EndAnchorPoint + DirectionBias - LengthBias;

	// Draw a line/spline
	DrawConnection(WireLayerID, StartPoint, EndPoint, Params);

	// Draw the arrow
	const FVector2f ArrowDrawPos = EndPoint - ArrowRadius;
	const float AngleInRadians = FMath::Atan2(DeltaPos.Y, DeltaPos.X);

	FSlateDrawElement::MakeRotatedBox(
		DrawElementsList,
		ArrowLayerID,
		FPaintGeometry(ArrowDrawPos, ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
		ArrowImage,
		ESlateDrawEffect::None,
		AngleInRadians,
		TOptional<FVector2f>(),
		FSlateDrawElement::RelativeToElement,
		Params.WireColor
		);
}

void FConversationGraphConnectionDrawingPolicy::DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params)
{
	// Get a reasonable seed point (halfway between the boxes)
	const FVector2f StartCenter = FGeometryHelper::CenterOf(StartGeom);
	const FVector2f EndCenter = FGeometryHelper::CenterOf(EndGeom);
	const FVector2f SeedPoint = (StartCenter + EndCenter) * 0.5f;

	// Find the (approximate) closest points between the two boxes
	const FVector2f StartAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(StartGeom, SeedPoint);
	const FVector2f EndAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(EndGeom, SeedPoint);

	DrawSplineWithArrow(StartAnchorPoint, EndAnchorPoint, Params);
}

FVector2f FConversationGraphConnectionDrawingPolicy::ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const
{
	const FVector2f Delta = End - Start;
	const FVector2f NormDelta = Delta.GetSafeNormal();

	return NormDelta;
}
