// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode.h"
#include "ConversationGraphSchema.h"
#include "ConversationGraphNode_Knot.h"

#include "ConversationGraphTypes.h"
#include "SConversationGraphNode.h"

#include "ConversationDatabase.h"
#include "ConversationCompiler.h"

#include "EdGraph/EdGraph.h"
#include "Kismet2/KismetEditorUtilities.h"

#include "ConversationNode.h"
#include "ConversationEditorColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode)

#define LOCTEXT_NAMESPACE "ConversationGraph"

UConversationGraphNode::UConversationGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UConversationGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	check(DesiredSchema);
	return DesiredSchema->GetClass()->IsChildOf(UConversationGraphSchema::StaticClass());
}

void UConversationGraphNode::FindDiffs(UEdGraphNode* OtherNode, FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);

	//@TODO: CONVERSATION: Diffing support
#if 0
	UBehaviorTreeGraphNode* OtherBTGraphNode = Cast<UBehaviorTreeGraphNode>(OtherNode);
	if (OtherBTGraphNode)
	{
		auto DiffSubNodes = [&Results](const FText& NodeTypeDisplayName, const TArray<UBehaviorTreeGraphNode*>& LhsSubNodes, const TArray<UBehaviorTreeGraphNode*>& RhsSubNodes)
		{
			TArray<FGraphDiffControl::FNodeMatch> NodeMatches;
			TSet<const UEdGraphNode*> MatchedRhsNodes;

			FGraphDiffControl::FNodeDiffContext AdditiveDiffContext;
			AdditiveDiffContext.NodeTypeDisplayName = NodeTypeDisplayName;
			AdditiveDiffContext.bIsRootNode = false;

			// march through the all the nodes in the rhs and look for matches 
			for (UEdGraphNode* RhsSubNode : RhsSubNodes)
			{
				FGraphDiffControl::FNodeMatch NodeMatch;
				NodeMatch.NewNode = RhsSubNode;

				// Do two passes, exact and soft
				for (UEdGraphNode* LhsSubNode : LhsSubNodes)
				{
					if (FGraphDiffControl::IsNodeMatch(LhsSubNode, RhsSubNode, true, &NodeMatches))
					{
						NodeMatch.OldNode = LhsSubNode;
						break;
					}
				}

				if (NodeMatch.NewNode == nullptr)
				{
					for (UEdGraphNode* LhsSubNode : LhsSubNodes)
					{
						if (FGraphDiffControl::IsNodeMatch(LhsSubNode, RhsSubNode, false, &NodeMatches))
						{
							NodeMatch.OldNode = LhsSubNode;
							break;
						}
					}
				}

				// if we found a corresponding node in the lhs graph, track it (so we can prevent future matches with the same nodes)
				if (NodeMatch.IsValid())
				{
					NodeMatches.Add(NodeMatch);
					MatchedRhsNodes.Add(NodeMatch.OldNode);
				}

				NodeMatch.Diff(AdditiveDiffContext, Results);
			}

			FGraphDiffControl::FNodeDiffContext SubtractiveDiffContext = AdditiveDiffContext;
			SubtractiveDiffContext.DiffMode = FGraphDiffControl::EDiffMode::Subtractive;
			SubtractiveDiffContext.DiffFlags = FGraphDiffControl::EDiffFlags::NodeExistance;

			// go through the lhs nodes to catch ones that may have been missing from the rhs graph
			for (UEdGraphNode* LhsSubNode : LhsSubNodes)
			{
				// if this node has already been matched, move on
				if (!LhsSubNode || MatchedRhsNodes.Find(LhsSubNode))
				{
					continue;
				}

				// There can't be a matching node in RhsGraph because it would have been found above
				FGraphDiffControl::FNodeMatch NodeMatch;
				NodeMatch.NewNode = LhsSubNode;

				NodeMatch.Diff(SubtractiveDiffContext, Results);
			}
		};

		DiffSubNodes(LOCTEXT("DecoratorDiffDisplayName", "Decorator"), Decorators, OtherBTGraphNode->Decorators);
		DiffSubNodes(LOCTEXT("ServiceDiffDisplayName", "Service"), Services, OtherBTGraphNode->Services);
	}
#endif
}

FName UConversationGraphNode::GetNameIcon() const
{
	if (const UConversationNode* RuntimeNode = Cast<const UConversationNode>(NodeInstance))
	{
		return RuntimeNode->GetNodeIconName();
	}

	return FName("BTEditor.Graph.BTNode.Icon");
}

bool UConversationGraphNode::IsOutBoundConnectionAllowed(const UConversationGraphNode* OtherNode, FText& OutErrorMessage) const
{
	if (!OtherNode)
	{
		return false;
	}

	if (const UConversationNodeWithLinks* MyTaskNode = GetRuntimeNode<UConversationNodeWithLinks>())
	{
		if (const UConversationNodeWithLinks* OtherTaskNode = OtherNode->GetRuntimeNode<UConversationNodeWithLinks>())
		{
			return MyTaskNode->IsOutBoundConnectionAllowed(OtherTaskNode, OutErrorMessage);
		}
	}

	return true;
}

bool UConversationGraphNode::IsOutBoundConnectionAllowed(const UConversationGraphNode_Knot* KnotNode, FText& OutErrorMessage) const
{
	if (!KnotNode)
	{
		return false;
	}

	if (const UConversationNodeWithLinks* MyTaskNode = GetRuntimeNode<UConversationNodeWithLinks>())
	{
		TArray<UConversationGraphNode*> GraphNodes;
		KnotNode->GatherAllOutBoundGraphNodes(GraphNodes);

		for (const UConversationGraphNode* OtherNode : GraphNodes)
		{
			if (const UConversationNodeWithLinks* OtherTaskNode = OtherNode->GetRuntimeNode<UConversationNodeWithLinks>())
			{
				if (!MyTaskNode->IsOutBoundConnectionAllowed(OtherTaskNode, OutErrorMessage))
				{
					return false;
				}
			}
		}
	}

	return true;
}

FText UConversationGraphNode::GetDescription() const
{
	if (const UConversationNode* RuntimeNode = Cast<const UConversationNode>(NodeInstance))
	{
		if (!RuntimeNode->ShowPropertyEditors())
		{
			return RuntimeNode->GetStaticDescription();
		}
		else
		{
			return FText::GetEmpty();
		}
	}

	return Super::GetDescription();
}

FText UConversationGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (const UConversationNode* RuntimeNode = Cast<const UConversationNode>(NodeInstance))
	{
		return RuntimeNode->GetDisplayNameText();
	}

	if (!ClassData.GetClassName().IsEmpty())
	{
		FString StoredClassName = ClassData.GetClassName();
		StoredClassName.RemoveFromEnd(TEXT("_C"));

		return FText::Format(NSLOCTEXT("ConversationGraph", "NodeClassError", "Class {0} not found, make sure it's saved!"), FText::FromString(StoredClassName));
	}

	return Super::GetNodeTitle(TitleType);
}

FLinearColor UConversationGraphNode::GetNodeBodyTintColor() const
{
	return ConversationEditorColors::NodeBody::Default;
}

UObject* UConversationGraphNode::GetJumpTargetForDoubleClick() const
{
	return (NodeInstance != nullptr) ? NodeInstance->GetClass() : nullptr;
}

bool UConversationGraphNode::CanJumpToDefinition() const
{
	return GetJumpTargetForDoubleClick() != nullptr;
}

void UConversationGraphNode::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);
	}

	//GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ReferencePath.GetAssetPathString());
}

void UConversationGraphNode::RequestRebuildConversation()
{
	FConversationCompiler::RebuildBank(Cast<UConversationDatabase>(GetGraph()->GetOuter()));
}

TSharedPtr<SGraphNode> UConversationGraphNode::CreateVisualWidget()
{
	return SNew(SConversationGraphNode, this);
}

#undef LOCTEXT_NAMESPACE

