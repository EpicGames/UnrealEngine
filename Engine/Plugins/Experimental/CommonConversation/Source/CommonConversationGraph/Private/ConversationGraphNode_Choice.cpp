// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode_Choice.h"
#include "ConversationEditorColors.h"
#include "ConversationChoiceNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode_Choice)

#define LOCTEXT_NAMESPACE "ConversationGraph"

UConversationGraphNode_Choice::UConversationGraphNode_Choice(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsSubNode = true;
}

void UConversationGraphNode_Choice::AllocateDefaultPins()
{
	// No pins for choices
}

FLinearColor UConversationGraphNode_Choice::GetNodeBodyTintColor() const
{
	return ConversationEditorColors::NodeBody::ChoiceColor;
}

FText UConversationGraphNode_Choice::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (NodeInstance)
	{
		UConversationChoiceNode* RuntimeNode = GetRuntimeNode<UConversationChoiceNode>();
		if (ensure(RuntimeNode))
		{
			const bool bIsDerivedType = RuntimeNode->GetClass() != UConversationChoiceNode::StaticClass();

			FFormatNamedArguments Args;
			Args.Add(TEXT("ChoiceName"), RuntimeNode->DefaultChoiceDisplayText);
#if WITH_EDITOR
			Args.Add(TEXT("ChoiceClassName"), RuntimeNode->GetClass()->GetDisplayNameText());
#else
			Args.Add(TEXT("ChoiceClassName"), FText::FromName(RuntimeNode->GetClass()->GetDisplayNameText()));
#endif

			const FText FormatString = (bIsDerivedType && !RuntimeNode->GetHideChoiceClassName()) ?
				LOCTEXT("UConversationGraphNode_Choice_GetNodeTitle_WithClass", "Choice \"{ChoiceName}\" ({ChoiceClassName})") :
				LOCTEXT("UConversationGraphNode_Choice_GetNodeTitle", "Choice \"{ChoiceName}\"");

			return FText::Format(FormatString, Args);
		}
	}
	
	return LOCTEXT("BrokenNode", "Broken Node");
}

#undef LOCTEXT_NAMESPACE
