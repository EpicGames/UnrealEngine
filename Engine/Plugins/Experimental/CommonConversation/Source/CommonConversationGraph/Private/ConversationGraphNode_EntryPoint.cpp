// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode_EntryPoint.h"
#include "ConversationGraphTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode_EntryPoint)

UConversationGraphNode_EntryPoint::UConversationGraphNode_EntryPoint(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UConversationGraphNode_EntryPoint::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UConversationGraphTypes::PinCategory_MultipleNodes, TEXT("Out"));
}

FName UConversationGraphNode_EntryPoint::GetNameIcon() const
{
	return FName("BTEditor.Graph.BTNode.Root.Icon");
}

