// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode_Knot.h"
#include "ConversationGraphNode.h"
#include "ConversationNode.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "SGraphNodeKnot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode_Knot)

#define LOCTEXT_NAMESPACE "ConversationGraph"

static const char* PC_Wildcard = "wildcard";

/////////////////////////////////////////////////////
// UConversationGraphNode_Knot

UConversationGraphNode_Knot::UConversationGraphNode_Knot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
}

void UConversationGraphNode_Knot::AllocateDefaultPins()
{
	const FName InputPinName(TEXT("InputPin"));
	const FName OutputPinName(TEXT("OutputPin"));

	UEdGraphPin* MyInputPin = CreatePin(EGPD_Input, PC_Wildcard, InputPinName);
	MyInputPin->bDefaultValueIsIgnored = true;

	UEdGraphPin* MyOutputPin = CreatePin(EGPD_Output, PC_Wildcard, OutputPinName);
}

FText UConversationGraphNode_Knot::GetTooltipText() const
{
	//@TODO: Should pull the tooltip from the source pin
	return LOCTEXT("KnotTooltip", "Reroute Node (reroutes wires)");
}

FText UConversationGraphNode_Knot::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(NodeComment);
	}
	else if (TitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("KnotListTitle", "Add Reroute Node...");
	}
	else
	{
		return LOCTEXT("KnotTitle", "Reroute Node");
	}
}

bool UConversationGraphNode_Knot::ShouldOverridePinNames() const
{
	return true;
}

FText UConversationGraphNode_Knot::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	// Keep the pin size tiny
	return FText::GetEmpty();
}

void UConversationGraphNode_Knot::OnRenameNode(const FString& NewName)
{
	NodeComment = NewName;
}

bool UConversationGraphNode_Knot::CanSplitPin(const UEdGraphPin* Pin) const
{
	return false;
}

TSharedPtr<class INameValidatorInterface> UConversationGraphNode_Knot::MakeNameValidator() const
{
	// Comments can be duplicated, etc...
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

UEdGraphPin* UConversationGraphNode_Knot::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	if(FromPin && Pins.Contains(FromPin))
	{
		return FromPin == Pins[0] ? Pins[1] : Pins[0];
	}

	return nullptr;
}

TSharedPtr<SGraphNode> UConversationGraphNode_Knot::CreateVisualWidget()
{
	return SNew(SGraphNodeKnot, this);
}

void UConversationGraphNode_Knot::GatherAllInBoundGraphNodes(TArray<UConversationGraphNode*>& OutGraphNodes) const
{
	TArray<const UConversationGraphNode_Knot*> VisitedKnots;
	GatherAllInBoundGraphNodes_Internal(OutGraphNodes, VisitedKnots);
}

void UConversationGraphNode_Knot::GatherAllOutBoundGraphNodes(TArray<UConversationGraphNode*>& OutGraphNodes) const
{
	TArray<const UConversationGraphNode_Knot*> VisitedKnots;
	GatherAllOutBoundGraphNodes_Internal(OutGraphNodes, VisitedKnots);
}

bool UConversationGraphNode_Knot::IsOutBoundConnectionAllowed(const UConversationGraphNode* OtherNode, FText& OutErrorMessage) const
{
	if (!OtherNode)
	{
		return false;
	}

	if (const UConversationNodeWithLinks* OtherTaskNode = OtherNode->GetRuntimeNode<UConversationNodeWithLinks>())
	{
		TArray<UConversationGraphNode*> GraphNodes;
		GatherAllInBoundGraphNodes(GraphNodes);

		for (const UConversationGraphNode* MyNode : GraphNodes)
		{
			if (const UConversationNodeWithLinks* MyTaskNode = MyNode->GetRuntimeNode<UConversationNodeWithLinks>())
			{
				if (!MyTaskNode->IsOutBoundConnectionAllowed(OtherTaskNode, OutErrorMessage))
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool UConversationGraphNode_Knot::IsOutBoundConnectionAllowed(const UConversationGraphNode_Knot* OtherKnotNode, FText& OutErrorMessage) const
{
	if (!OtherKnotNode)
	{
		return false;
	}

	TArray<UConversationGraphNode*> InGraphNodes;
	GatherAllInBoundGraphNodes(InGraphNodes);
	OtherKnotNode->GatherAllInBoundGraphNodes(InGraphNodes);

	TArray<UConversationGraphNode*> OutGraphNodes;
	GatherAllOutBoundGraphNodes(OutGraphNodes);
	OtherKnotNode->GatherAllOutBoundGraphNodes(OutGraphNodes);

	for (const UConversationGraphNode* MyNode : InGraphNodes)
	{
		if (const UConversationNodeWithLinks* MyTaskNode = MyNode->GetRuntimeNode<UConversationNodeWithLinks>())
		{
			for (const UConversationGraphNode* OtherNode : OutGraphNodes)
			{
				if (const UConversationNodeWithLinks* OtherTaskNode = OtherNode->GetRuntimeNode<UConversationNodeWithLinks>())
				{
					if (!MyTaskNode->IsOutBoundConnectionAllowed(OtherTaskNode, OutErrorMessage))
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

void UConversationGraphNode_Knot::GatherAllInBoundGraphNodes_Internal(TArray<UConversationGraphNode*>& OutGraphNodes, TArray<const UConversationGraphNode_Knot*>& VisitedKnots) const
{
	if (VisitedKnots.Contains(this))
	{
		return;
	}
	VisitedKnots.Add(this);

	for (UEdGraphPin* LinkedPin : GetInputPin()->LinkedTo)
	{
		if (UConversationGraphNode* GraphNode = Cast<UConversationGraphNode>(LinkedPin->GetOwningNode()))
		{
			OutGraphNodes.Add(GraphNode);
		}
		else if (UConversationGraphNode_Knot* GraphKnot = Cast<UConversationGraphNode_Knot>(LinkedPin->GetOwningNode()))
		{
			GraphKnot->GatherAllInBoundGraphNodes_Internal(OutGraphNodes, VisitedKnots);
		}
	}
}

void UConversationGraphNode_Knot::GatherAllOutBoundGraphNodes_Internal(TArray<UConversationGraphNode*>& OutGraphNodes, TArray<const UConversationGraphNode_Knot*>& VisitedKnots) const
{
	if (VisitedKnots.Contains(this))
	{
		return;
	}
	VisitedKnots.Add(this);

	for (UEdGraphPin* LinkedPin : GetOutputPin()->LinkedTo)
	{
		if (UConversationGraphNode* GraphNode = Cast<UConversationGraphNode>(LinkedPin->GetOwningNode()))
		{
			OutGraphNodes.Add(GraphNode);
		}
		else if (UConversationGraphNode_Knot* GraphKnot = Cast<UConversationGraphNode_Knot>(LinkedPin->GetOwningNode()))
		{
			GraphKnot->GatherAllOutBoundGraphNodes_Internal(OutGraphNodes, VisitedKnots);
		}
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

