// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode_Requirement.h"
#include "ConversationEditorColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode_Requirement)

UConversationGraphNode_Requirement::UConversationGraphNode_Requirement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsSubNode = true;
}

void UConversationGraphNode_Requirement::AllocateDefaultPins()
{
	// No pins for requirements
}

FLinearColor UConversationGraphNode_Requirement::GetNodeBodyTintColor() const
{
	return ConversationEditorColors::NodeBody::RequirementColor;
}

