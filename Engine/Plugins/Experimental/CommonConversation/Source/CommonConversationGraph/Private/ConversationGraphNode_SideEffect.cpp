// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode_SideEffect.h"
#include "ConversationEditorColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode_SideEffect)

UConversationGraphNode_SideEffect::UConversationGraphNode_SideEffect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsSubNode = true;
}

void UConversationGraphNode_SideEffect::AllocateDefaultPins()
{
	// No pins for side effects
}

FLinearColor UConversationGraphNode_SideEffect::GetNodeBodyTintColor() const
{
	return ConversationEditorColors::NodeBody::SideEffectColor;
}

