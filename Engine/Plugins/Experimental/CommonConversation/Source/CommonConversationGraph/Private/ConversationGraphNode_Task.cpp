// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphNode_Task.h"
#include "ConversationTaskNode.h"
#include "ConversationGraphTypes.h"
#include "ConversationGraphSchema.h"
#include "EdGraph/EdGraph.h"
#include "SGraphEditorActionMenuAI.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphNode_Task)

#define LOCTEXT_NAMESPACE "ConversationGraph"

UConversationGraphNode_Task::UConversationGraphNode_Task(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UConversationGraphNode_Task::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UConversationGraphTypes::PinCategory_MultipleNodes, TEXT("In"));
	CreatePin(EGPD_Output, UConversationGraphTypes::PinCategory_MultipleNodes, TEXT("Out"));
}

FText UConversationGraphNode_Task::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (const UConversationNode* MyNode = Cast<UConversationNode>(NodeInstance))
	{
		return MyNode->GetDisplayNameText();
	}

	return Super::GetNodeTitle(TitleType);
}

FLinearColor UConversationGraphNode_Task::GetNodeBodyTintColor() const
{
	FLinearColor Result = FLinearColor::Red;
	if (NodeInstance)
	{
		if (!GetRuntimeNode<UConversationTaskNode>()->GetNodeBodyColor(/*out*/ Result))
		{
			Result = Super::GetNodeBodyTintColor();
		}
	}
	return Result;
}

void UConversationGraphNode_Task::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	TWeakObjectPtr<UConversationGraphNode> WeakThis(const_cast<UConversationGraphNode_Task*>(this));

	auto CreateAddRequirementSubMenu = [WeakThis](FMenuBuilder& MenuBuilder, UEdGraph* Graph)
	{
		TSharedRef<SGraphEditorActionMenuAI> Menu =
			SNew(SGraphEditorActionMenuAI)
			.GraphObj(Graph)
			.GraphNode(WeakThis.Get())
			.SubNodeFlags((int32)EConversationGraphSubNodeType::Requirement)
			.AutoExpandActionMenu(true);

		MenuBuilder.AddWidget(Menu, FText(), true);
	};

	FToolMenuSection& Section = Menu->FindOrAddSection("ConversationGraphNode_Task");
	const UConversationGraphSchema* Schema = Context->Graph ? Cast<const UConversationGraphSchema>(Context->Graph->GetSchema()) : nullptr;

	if (!Schema || Schema->HasSubNodeClasses(int32(EConversationGraphSubNodeType::Requirement)))
	{
		Section.AddSubMenu(
			"AddRequirement",
			LOCTEXT("AddRequirement", "Add Requirement..."),
			LOCTEXT("AddRequirementTooltip", "Adds a new requirement as a subnode"),
			FNewMenuDelegate::CreateLambda(CreateAddRequirementSubMenu, (UEdGraph*)Context->Graph));
	}


	auto CreateAddSideEffectSubMenu = [WeakThis](FMenuBuilder& MenuBuilder, UEdGraph* Graph)
	{
		TSharedRef<SGraphEditorActionMenuAI> Menu =
			SNew(SGraphEditorActionMenuAI)
			.GraphObj(Graph)
			.GraphNode(WeakThis.Get())
			.SubNodeFlags((int32)EConversationGraphSubNodeType::SideEffect)
			.AutoExpandActionMenu(true);

		MenuBuilder.AddWidget(Menu, FText(), true);
	};

	if (!Schema || Schema->HasSubNodeClasses(int32(EConversationGraphSubNodeType::SideEffect)))
	{
		Section.AddSubMenu(
			"AddSideEffect",
			LOCTEXT("AddSideEffect", "Add Side Effect..."),
			LOCTEXT("AddSideEffectTooltip", "Adds a new side effect as a subnode"),
			FNewMenuDelegate::CreateLambda(CreateAddSideEffectSubMenu, (UEdGraph*)Context->Graph));
	}


	auto CreateAddChoiceSubMenu = [WeakThis](FMenuBuilder& MenuBuilder, UEdGraph* Graph)
	{
		TSharedRef<SGraphEditorActionMenuAI> Menu =
			SNew(SGraphEditorActionMenuAI)
			.GraphObj(Graph)
			.GraphNode(WeakThis.Get())
			.SubNodeFlags((int32)EConversationGraphSubNodeType::Choice)
			.AutoExpandActionMenu(true);

		MenuBuilder.AddWidget(Menu, FText(), true);
	};

	if (!Schema || Schema->HasSubNodeClasses(int32(EConversationGraphSubNodeType::Choice)))
	{
		Section.AddSubMenu(
			"AddChoice",
			LOCTEXT("AddChoice", "Add Choice..."),
			LOCTEXT("AddChoiceTooltip", "Adds a new choice as a subnode"),
			FNewMenuDelegate::CreateLambda(CreateAddChoiceSubMenu, (UEdGraph*)Context->Graph));
	}
}

void UConversationGraphNode_Task::ApplyAddRequirementMenu(FMenuBuilder& MenuBuilder, UEdGraph* Graph)
{
	TWeakObjectPtr<UConversationGraphNode> WeakThis(const_cast<UConversationGraphNode_Task*>(this));

	TSharedRef<SGraphEditorActionMenuAI> Menu =
		SNew(SGraphEditorActionMenuAI)
		.GraphObj(Graph)
		.GraphNode(WeakThis.Get())
		.SubNodeFlags((int32)EConversationGraphSubNodeType::Requirement)
		.AutoExpandActionMenu(true);

	MenuBuilder.AddWidget(Menu, FText(), true);
}

#undef LOCTEXT_NAMESPACE
