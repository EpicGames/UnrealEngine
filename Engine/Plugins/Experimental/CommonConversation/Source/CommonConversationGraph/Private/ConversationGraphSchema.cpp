// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphSchema.h"
#include "ConversationGraphTypes.h"
#include "ConversationGraphConnectionDrawingPolicy.h"

#include "ConversationEntryPointNode.h"
#include "ConversationGraphNode_EntryPoint.h"

#include "ConversationTaskNode.h"
#include "ConversationGraphNode_Task.h"

#include "ConversationRequirementNode.h"
#include "ConversationGraphNode_Requirement.h"

#include "ConversationSideEffectNode.h"
#include "ConversationGraphNode_SideEffect.h"

#include "ConversationChoiceNode.h"
#include "ConversationGraphNode_Choice.h"
#include "ConversationGraphNode_Knot.h"

#include "BlueprintActionDatabase.h"
#include "EdGraph/EdGraph.h"
#include "GraphEditorActions.h"
#include "ToolMenu.h"
#include "ScopedTransaction.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphSchema)

#define LOCTEXT_NAMESPACE "ConversationEditor"

namespace ConversationEditorCVar
{
	static bool CheckForCyclesCVar = true;
	FAutoConsoleVariableRef CVarCheckForCycles(
		TEXT("ConversationEditor.CheckForCycles"),
		CheckForCyclesCVar,
		TEXT("This cvar controles if the Conversation Editor should check for cycles when links are created.\n")
		TEXT("0: Don't Check, 1: Check for Cycles (Default)"),
		ECVF_Default);

	static bool DisallowMultipleRerouteNodeOutputLinksCVar = false;
	FAutoConsoleVariableRef CVarDisallowMultipleRerouteNodeOutputLinks(
		TEXT("ConversationEditor.DisallowMultipleRerouteNodeOutputLinks"),
		DisallowMultipleRerouteNodeOutputLinksCVar,
		TEXT("Disallows Reroute nodes from visually splitting output links in Conversation Editor graph. Split links result in leftmost link always executing.\n")
		TEXT("0: Allow multiple output links (Default), 1: Disallow multiple output links"),
		ECVF_Default);
}

TSharedPtr<FGraphNodeClassHelper> ConversationClassCache;

FGraphNodeClassHelper& GetConversationClassCache()
{
	if (!ConversationClassCache.IsValid())
	{
		ConversationClassCache = MakeShareable(new FGraphNodeClassHelper(UConversationNode::StaticClass()));
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UConversationTaskNode::StaticClass());
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UConversationEntryPointNode::StaticClass());
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UConversationSideEffectNode::StaticClass());
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UConversationRequirementNode::StaticClass());
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UConversationChoiceNode::StaticClass());
		ConversationClassCache->UpdateAvailableBlueprintClasses();
	}

	return *ConversationClassCache.Get();
}

bool IsConnectionAllowed(const UEdGraphPin* PinA, const UEdGraphPin* PinB, FText& OutErrorMessage)
{
	if (!PinA || !PinB)
	{
		return false;
	}

	const UConversationGraphNode* PinAGraphNode = Cast<UConversationGraphNode>(PinA->GetOwningNode());
	const UConversationGraphNode_Knot* PinAKnot = Cast<UConversationGraphNode_Knot>(PinA->GetOwningNode());
	const UConversationGraphNode* PinBGraphNode = Cast<UConversationGraphNode>(PinB->GetOwningNode());
	const UConversationGraphNode_Knot* PinBKnot = Cast<UConversationGraphNode_Knot>(PinB->GetOwningNode());

	// If both are GraphNode
	if(PinAGraphNode && PinBGraphNode)
	{
		if (PinA->Direction == EGPD_Output)
		{
			return PinAGraphNode->IsOutBoundConnectionAllowed(PinBGraphNode, OutErrorMessage);
		}
		else if (PinB->Direction == EGPD_Output)
		{
			return PinBGraphNode->IsOutBoundConnectionAllowed(PinAGraphNode, OutErrorMessage);
		}
	}
	// If both are Knot, direction does not matter
	else if (PinAKnot && PinBKnot)
	{
		return PinAKnot->IsOutBoundConnectionAllowed(PinBKnot, OutErrorMessage);
	}
	// If one is GraphNode and one is Knot
	else
	{
		if (PinA->Direction == EGPD_Output)
		{
			if (PinAGraphNode && PinBKnot)
			{
				return PinAGraphNode->IsOutBoundConnectionAllowed(PinBKnot, OutErrorMessage);
			}
			else if (PinAKnot && PinBGraphNode)
			{
				return PinAKnot->IsOutBoundConnectionAllowed(PinBGraphNode, OutErrorMessage);
			}
		}
		else if (PinB->Direction == EGPD_Output)
		{
			if (PinBGraphNode && PinAKnot)
			{
				return PinBGraphNode->IsOutBoundConnectionAllowed(PinAKnot, OutErrorMessage);
			}
			else if (PinBKnot && PinAGraphNode)
			{
				return PinBKnot->IsOutBoundConnectionAllowed(PinAGraphNode, OutErrorMessage);
			}
		}
	}

	return true;
}

//////////////////////////////////////////////////////////////////////
//

UEdGraphNode* FConversationGraphSchemaAction_AutoArrange::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
// 	if (UBehaviorTreeGraph* Graph = Cast<UBehaviorTreeGraph>(ParentGraph))
// 	{
// 		Graph->AutoArrange();
// 	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////
// UConversationGraphSchema

int32 UConversationGraphSchema::CurrentCacheRefreshID = 0;

UConversationGraphSchema::UConversationGraphSchema(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UConversationGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	//@TODO: CONVERSATION: Add an entry point by default
// 	FGraphNodeCreator<UConversationGraphNode_EntryPoint> NodeCreator(Graph);
// 	UConversationGraphNode_EntryPoint* MyNode = NodeCreator.CreateNode();
// 	NodeCreator.Finalize();
// 	SetNodeMetaData(MyNode, FNodeMetadata::DefaultGraphNode);
}

void UConversationGraphSchema::GetGraphNodeContextActions(FGraphContextMenuBuilder& ContextMenuBuilder, int32 SubNodeFlags) const
{
	Super::GetGraphNodeContextActions(ContextMenuBuilder, SubNodeFlags);
}

bool UConversationGraphSchema::HasSubNodeClasses(int32 SubNodeFlags) const
{
	TArray<FGraphNodeClassData> TempClassData;
	UClass* TempClass = nullptr;
	GetSubNodeClasses(SubNodeFlags, TempClassData, TempClass);
	return !TempClassData.IsEmpty();
}

void UConversationGraphSchema::GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const
{
	FGraphNodeClassHelper& ClassCache = GetConversationClassCache();
	TArray<FGraphNodeClassData> TempClassData;

	switch ((EConversationGraphSubNodeType)SubNodeFlags)
	{
	case EConversationGraphSubNodeType::Requirement:
		ClassCache.GatherClasses(UConversationRequirementNode::StaticClass(), /*out*/ TempClassData);
		GraphNodeClass = UConversationGraphNode_Requirement::StaticClass();
		break;
	case EConversationGraphSubNodeType::SideEffect:
		ClassCache.GatherClasses(UConversationSideEffectNode::StaticClass(), /*out*/ TempClassData);
		GraphNodeClass = UConversationGraphNode_SideEffect::StaticClass();
		break;
	case EConversationGraphSubNodeType::Choice:
		ClassCache.GatherClasses(UConversationChoiceNode::StaticClass(), /*out*/ TempClassData);
		GraphNodeClass = UConversationGraphNode_Choice::StaticClass();
		break;
	default:
		unimplemented();
	}

	for (FGraphNodeClassData& Class : TempClassData)
	{
		bool bIsAllowed = false;
		// We check the name only first to test the allowed status without possibly loading a full uasset class from disk
		// If there is no package name, fallback to testing with a fully loaded class
		if (!Class.GetPackageName().IsEmpty())
		{
			bIsAllowed = FBlueprintActionDatabase::IsClassAllowed(FTopLevelAssetPath(FName(Class.GetPackageName()), FName(Class.GetClassName())), FBlueprintActionDatabase::EPermissionsContext::Node);
		}
		else
		{
			bIsAllowed = FBlueprintActionDatabase::IsClassAllowed(Class.GetClass(), FBlueprintActionDatabase::EPermissionsContext::Node);
		}

		if (bIsAllowed)
		{
			ClassData.Add(std::move(Class));
		}
	}
}

void UConversationGraphSchema::AddConversationNodeOptions(const FString& CategoryName, FGraphContextMenuBuilder& ContextMenuBuilder, TSubclassOf<UConversationNode> RuntimeNodeType, TSubclassOf<UConversationGraphNode> EditorNodeType) const
{
	FCategorizedGraphActionListBuilder ListBuilder(CategoryName);

	TArray<FGraphNodeClassData> NodeClasses;
	GetConversationClassCache().GatherClasses(RuntimeNodeType, /*out*/ NodeClasses);

	for (FGraphNodeClassData& NodeClass : NodeClasses)
	{
		bool bIsAllowed = false;
		// We check the name only first to test the allowed status without possibly loading a full uasset class from disk
		// If there is no package name, fallback to testing with a fully loaded class
		if (!NodeClass.GetPackageName().IsEmpty())
		{
			bIsAllowed = FBlueprintActionDatabase::IsClassAllowed(FTopLevelAssetPath(FName(NodeClass.GetPackageName()), FName(NodeClass.GetClassName())), FBlueprintActionDatabase::EPermissionsContext::Node);
		}
		else
		{
			bIsAllowed = FBlueprintActionDatabase::IsClassAllowed(NodeClass.GetClass(), FBlueprintActionDatabase::EPermissionsContext::Node);
		}
		
		if (bIsAllowed)
		{
			const FText NodeTypeName = FText::FromString(FName::NameToDisplayString(NodeClass.ToString(), false));

			TSharedPtr<FAISchemaAction_NewNode> AddOpAction = UAIGraphSchema::AddNewNodeAction(ListBuilder, NodeClass.GetCategory(), NodeTypeName, FText::GetEmpty());

			UConversationGraphNode* OpNode = NewObject<UConversationGraphNode>(ContextMenuBuilder.OwnerOfTemporaries, EditorNodeType);
			OpNode->ClassData = NodeClass;
			AddOpAction->NodeTemplate = OpNode;
		}
	}

	ContextMenuBuilder.Append(ListBuilder);
}

void UConversationGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const FName PinCategory = ContextMenuBuilder.FromPin ?
		ContextMenuBuilder.FromPin->PinType.PinCategory :
		UConversationGraphTypes::PinCategory_MultipleNodes;

	const bool bNoParent = (ContextMenuBuilder.FromPin == NULL);
	const bool bOnlyTasks = (PinCategory == UConversationGraphTypes::PinCategory_SingleTask);
	const bool bOnlyComposites = (PinCategory == UConversationGraphTypes::PinCategory_SingleComposite);
	const bool bAllowComposites = bNoParent || !bOnlyTasks || bOnlyComposites;
	const bool bAllowTasks = bNoParent || !bOnlyComposites || bOnlyTasks;

	FGraphNodeClassHelper& ClassCache = GetConversationClassCache();

	if (bAllowTasks)
	{
		AddConversationNodeOptions(TEXT("Tasks"), ContextMenuBuilder, UConversationTaskNode::StaticClass(), UConversationGraphNode_Task::StaticClass());
	}

	if (bNoParent || (ContextMenuBuilder.FromPin && (ContextMenuBuilder.FromPin->Direction == EGPD_Input)))
	{
		AddConversationNodeOptions(TEXT("Entry Point"), ContextMenuBuilder, UConversationEntryPointNode::StaticClass(), UConversationGraphNode_EntryPoint::StaticClass());
	}

	if (bNoParent)
	{
		TSharedPtr<FConversationGraphSchemaAction_AutoArrange> Action( new FConversationGraphSchemaAction_AutoArrange(FText::GetEmpty(), LOCTEXT("AutoArrange", "Auto Arrange"), FText::GetEmpty(), 0) );
		ContextMenuBuilder.AddAction(Action);
	}
}

void UConversationGraphSchema::GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node && !Context->Pin)
	{
		const UConversationGraphNode* ConversationGraphNode = Cast<const UConversationGraphNode>(Context->Node);
		if (ConversationGraphNode && ConversationGraphNode->CanPlaceBreakpoints())
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaBreakpoints", LOCTEXT("BreakpointsHeader", "Breakpoints"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().ToggleBreakpoint);
			Section.AddMenuEntry(FGraphEditorCommands::Get().AddBreakpoint);
			Section.AddMenuEntry(FGraphEditorCommands::Get().RemoveBreakpoint);
			Section.AddMenuEntry(FGraphEditorCommands::Get().EnableBreakpoint);
			Section.AddMenuEntry(FGraphEditorCommands::Get().DisableBreakpoint);
		}
	}

	Super::GetContextMenuActions(Menu, Context);
}

const FPinConnectionResponse UConversationGraphSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	if (PinA == nullptr || PinB == nullptr)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinNull", "One or Both of the pins was null"));
	}

	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorSameNode", "Both are on the same node"));
	}

	// Check that both links are owned with a valid node class before using the class
	if (!PinA->GetOwningNode())
	{
		if (PinA->Direction == EGPD_Input)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("InputNodeTypeUnrecognized", "Input node type undefined"));
		}
		else if(PinA->Direction == EGPD_Output)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("OutputNodeTypeUnrecognized", "Output node type undefined"));
		}
		else
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("NodeTypeUnrecognized", "Owning node type undefined"));
		}
	}

	if (!PinB->GetOwningNode())
	{
		if (PinB->Direction == EGPD_Input)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("InputNodeTypeUnrecognized", "Input node type undefined"));
		}
		else if (PinB->Direction == EGPD_Output)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("OutputNodeTypeUnrecognized", "Output node type undefined"));
		}
		else
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("NodeTypeUnrecognized", "Owning node type undefined"));
		}
	}

	const bool bPinAIsSingleComposite = (PinA->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleComposite);
	const bool bPinAIsSingleTask = (PinA->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleTask);
	const bool bPinAIsSingleNode = (PinA->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleNode);

	const bool bPinBIsSingleComposite = (PinB->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleComposite);
	const bool bPinBIsSingleTask = (PinB->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleTask);
	const bool bPinBIsSingleNode = (PinB->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleNode);

	const bool bPinAIsTask = PinA->GetOwningNode()->IsA(UConversationGraphNode_Task::StaticClass());
	const bool bPinAIsComposite = false;// PinA->GetOwningNode()->IsA(UConversationGraphNode_Composite::StaticClass());

	const bool bPinBIsTask = PinB->GetOwningNode()->IsA(UConversationGraphNode_Task::StaticClass());
	const bool bPinBIsComposite = false;// PinB->GetOwningNode()->IsA(UConversationGraphNode_Composite::StaticClass());

	if ((bPinAIsSingleComposite && !bPinBIsComposite) || (bPinBIsSingleComposite && !bPinAIsComposite))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorOnlyComposite", "Only composite nodes are allowed"));
	}

	if ((bPinAIsSingleTask && !bPinBIsTask) || (bPinBIsSingleTask && !bPinAIsTask))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorOnlyTask", "Only task nodes are allowed"));
	}

	// Compare the directions
	if ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Input))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorInput", "Can't connect input node to input node"));
	}
	else if ((PinB->Direction == EGPD_Output) && (PinA->Direction == EGPD_Output))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorOutput", "Can't connect output node to output node"));
	}

	class FNodeVisitorCycleChecker
	{
	public:
		/** Check whether a loop in the graph would be caused by linking the passed-in nodes */
		bool CheckForLoop(UEdGraphNode* StartNode, UEdGraphNode* EndNode)
		{
			VisitedNodes.Add(EndNode);
			return TraverseInputNodesToRoot(StartNode);
		}

	private:
		/**
		 * Helper function for CheckForLoop()
		 * @param	Node	The node to start traversal at
		 * @return true if we reached a root node (i.e. a node with no input pins), false if we encounter a node we have already seen
		 */
		bool TraverseInputNodesToRoot(UEdGraphNode* Node)
		{
			VisitedNodes.Add(Node);

			// Follow every input pin until we cant any more ('root') or we reach a node we have seen (cycle)
			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* MyPin = Node->Pins[PinIndex];

				if (MyPin->Direction == EGPD_Input)
				{
					for (int32 LinkedPinIndex = 0; LinkedPinIndex < MyPin->LinkedTo.Num(); ++LinkedPinIndex)
					{
						UEdGraphPin* OtherPin = MyPin->LinkedTo[LinkedPinIndex];
						if (OtherPin)
						{
							UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
							if (VisitedNodes.Contains(OtherNode))
							{
								return false;
							}
							else
							{
								return TraverseInputNodesToRoot(OtherNode);
							}
						}
					}
				}
			}

			return true;
		}

		TSet<UEdGraphNode*> VisitedNodes;
	};

	if (ConversationEditorCVar::CheckForCyclesCVar)
	{
		// check for cycles
		FNodeVisitorCycleChecker CycleChecker;
		if (!CycleChecker.CheckForLoop(PinA->GetOwningNode(), PinB->GetOwningNode()))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorcycle", "Can't create a graph cycle"));
		}
	}

	// Check if the connection is allowed by the tasks
	FText ErrorMessage;
	if (!IsConnectionAllowed(PinA, PinB, ErrorMessage))
	{
		if (ErrorMessage.IsEmpty())
		{
			ErrorMessage = LOCTEXT("DefaultConnectionNotAllowed", "The connection between these nodes is not allowed");
		}
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, MoveTemp(ErrorMessage));
	}

	const bool bPinASingleLink = bPinAIsSingleComposite || bPinAIsSingleTask || bPinAIsSingleNode;
	const bool bPinBSingleLink = bPinBIsSingleComposite || bPinBIsSingleTask || bPinBIsSingleNode;

	// Joint Rules For Pins
	//----------------------------------
	//PinB is receiving input from other sources
	if (PinB->Direction == EGPD_Input && PinB->LinkedTo.Num() > 0)
	{
		// PinA is exclusive output
		if (bPinASingleLink)
		{
			// break all previous links between both nodes
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, LOCTEXT("PinConnectReplace", "Replace connection"));
		}
	}
	else if (PinA->Direction == EGPD_Input && PinA->LinkedTo.Num() > 0)
	{
		// Pin B is exclusive output
		if (bPinBSingleLink)
		{
			// break all previous links between both nodes
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, LOCTEXT("PinConnectReplace", "Replace connection"));
		}
	}

	// Singular Rules For Pins
	//------------------------------
	// Reroute Nodes have a single output link
	// Not the same as being a SingleLink. Receiving nodes are still inclusive w/ unrestricted inputs

	if(ConversationEditorCVar::DisallowMultipleRerouteNodeOutputLinksCVar)
	{ 
		if (PinA->GetOwningNode()->IsA(UConversationGraphNode_Knot::StaticClass()) && PinA->Direction == EGPD_Output)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("PinRerouteOutputOverride", "Reroute node limited to 1 output link"));
		}

		if (PinB->GetOwningNode()->IsA(UConversationGraphNode_Knot::StaticClass()) && PinB->Direction == EGPD_Output)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, LOCTEXT("PinRerouteOutputOverride", "Reroute node limited to 1 output link"));
		}
	}

	// Pin A is an exclusive link and is already linked to other sources
	if (bPinASingleLink && PinA->LinkedTo.Num() > 0)
	{
		// break all previous links to pin A
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("PinConnectReplace", "Replace connection"));
	}
	else if (bPinBSingleLink && PinB->LinkedTo.Num() > 0)
	{
		// Pin B is an exclusive link and is already linked to other sources
		// break all previous links to pin B
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, LOCTEXT("PinConnectReplace", "Replace connection"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, LOCTEXT("PinConnect", "Connect nodes"));
}

const FPinConnectionResponse UConversationGraphSchema::CanMergeNodes(const UEdGraphNode* NodeA, const UEdGraphNode* NodeB) const
{
	// Make sure the nodes are not the same 
	if (NodeA == NodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are the same node"));
	}

	const bool bIsSubnode_A = Cast<UConversationGraphNode>(NodeA) && Cast<UConversationGraphNode>(NodeA)->IsSubNode();
	const bool bIsSubnode_B = Cast<UConversationGraphNode>(NodeB) && Cast<UConversationGraphNode>(NodeB)->IsSubNode();
	const bool bIsTask_B = NodeB->IsA(UConversationGraphNode_Task::StaticClass());

	if (bIsSubnode_A && (bIsSubnode_B || bIsTask_B))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

void UConversationGraphSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(LOCTEXT("CreateRerouteNodeOnWire", "Create Reroute Node"));

	//@TODO: This constant is duplicated from inside of SGraphNodeKnot
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	// Create a new knot
	UEdGraph* OwningGraph = PinA->GetOwningNode()->GetGraph();

	if (ensure(OwningGraph))
	{
		FGraphNodeCreator<UConversationGraphNode_Knot> NodeCreator(*OwningGraph);
		UConversationGraphNode_Knot* MyNode = NodeCreator.CreateNode();
		MyNode->NodePosX = KnotTopLeft.X;
		MyNode->NodePosY = KnotTopLeft.Y;
		//MyNode->SnapToGrid(SNAP_GRID);
	 	NodeCreator.Finalize();

		//UK2Node_Knot* NewKnot = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Knot>(ParentGraph, KnotTopLeft, EK2NewNodeFlags::SelectNewNode);

		// Move the connections across (only notifying the knot, as the other two didn't really change)
		PinA->BreakLinkTo(PinB);
		PinA->MakeLinkTo((PinA->Direction == EGPD_Output) ? CastChecked<UConversationGraphNode_Knot>(MyNode)->GetInputPin() : CastChecked<UConversationGraphNode_Knot>(MyNode)->GetOutputPin());
		PinB->MakeLinkTo((PinB->Direction == EGPD_Output) ? CastChecked<UConversationGraphNode_Knot>(MyNode)->GetInputPin() : CastChecked<UConversationGraphNode_Knot>(MyNode)->GetOutputPin());
	}
}

FLinearColor UConversationGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FColor::White;
}

class FConnectionDrawingPolicy* UConversationGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FConversationGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

bool UConversationGraphSchema::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UConversationGraphSchema::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UConversationGraphSchema::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}

//////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

