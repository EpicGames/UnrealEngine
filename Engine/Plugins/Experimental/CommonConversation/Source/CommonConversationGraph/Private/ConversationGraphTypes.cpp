// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationGraphTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationGraphTypes)

const FName UConversationGraphTypes::PinCategory_MultipleNodes("MultipleNodes");
const FName UConversationGraphTypes::PinCategory_SingleComposite("SingleComposite");
const FName UConversationGraphTypes::PinCategory_SingleTask("SingleTask");
const FName UConversationGraphTypes::PinCategory_SingleNode("SingleNode");

UConversationGraphTypes::UConversationGraphTypes(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

