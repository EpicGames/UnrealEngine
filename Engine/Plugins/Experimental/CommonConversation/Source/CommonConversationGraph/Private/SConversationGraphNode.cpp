// Copyright Epic Games, Inc. All Rights Reserved.


#include "SConversationGraphNode.h"
#include "ConversationGraphTypes.h"
#include "Widgets/Images/SImage.h"
#include "EdGraph/EdGraph.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SToolTip.h"

#include "ConversationGraphNode_Requirement.h"
#include "ConversationGraphNode_Choice.h"
#include "ConversationGraphNode_SideEffect.h"
#include "ConversationGraphNode_EntryPoint.h"

#include "Editor.h"
//#include "ConversationDebugger.h"
#include "GraphEditorSettings.h"
#include "SGraphPanel.h"
#include "SCommentBubble.h"
#include "NodeFactory.h"
#include "ConversationEditorColors.h"
#include "IDocumentation.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SLevelOfDetailBranchNode.h"
#include "ConversationDatabase.h"
#include "ConversationGraphNode_Task.h"
#include "ConversationTaskNode.h"
#include "IPropertyRowGenerator.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Widgets/Layout/SGridPanel.h"
#include "PropertyHandle.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConversationEditor"

namespace
{
	static const bool bShowExecutionIndexInEditorMode = false;
}

namespace ConversationEditorCVar
{
	static bool SelectWholeNodeOnDropCVar = false;
	FAutoConsoleVariableRef CVarSelectWholeNodeOnDrop(
		TEXT("ConversationEditor.SelectWholeNodeOnDrop"),
		SelectWholeNodeOnDropCVar,
		TEXT("This cvar controles whether the subnode or whole node will be selected when a subnode is dropped onto the node.\n")
		TEXT("0: SubNode will remain selected (Default), 1: Whole Node will be selected"),
		ECVF_Default);
}

/////////////////////////////////////////////////////
// SConversationPin

class SConversationPin : public SGraphPinAI
{
public:
	SLATE_BEGIN_ARGS(SConversationPin){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);
protected:
	/** @return The color that we should use to draw this pin */
	virtual FSlateColor GetPinColor() const override;
};

void SConversationPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPinAI::Construct(SGraphPinAI::FArguments(), InPin);
}

FSlateColor SConversationPin::GetPinColor() const
{
	return 
		bIsDiffHighlighted ? ConversationEditorColors::Pin::Diff :
		IsHovered() ? ConversationEditorColors::Pin::Hover :
		(GraphPinObj->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleComposite) ? ConversationEditorColors::Pin::CompositeOnly :
		(GraphPinObj->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleTask) ? ConversationEditorColors::Pin::TaskOnly :
		(GraphPinObj->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleNode) ? ConversationEditorColors::Pin::SingleNode :
		ConversationEditorColors::Pin::Default;
}

/** Widget for overlaying an execution-order index onto a node */
class SConversationIndex : public SCompoundWidget
{
public:
	/** Delegate event fired when the hover state of this widget changes */
	DECLARE_DELEGATE_OneParam(FOnHoverStateChanged, bool /* bHovered */);

	/** Delegate used to receive the color of the node, depending on hover state and state of other siblings */
	DECLARE_DELEGATE_RetVal_OneParam(FSlateColor, FOnGetIndexColor, bool /* bHovered */);

	SLATE_BEGIN_ARGS(SConversationIndex){}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_EVENT(FOnHoverStateChanged, OnHoverStateChanged)
		SLATE_EVENT(FOnGetIndexColor, OnGetIndexColor)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		OnHoverStateChangedEvent = InArgs._OnHoverStateChanged;
		OnGetIndexColorEvent = InArgs._OnGetIndexColor;

		const FSlateBrush* IndexBrush = FAppStyle::GetBrush(TEXT("BTEditor.Graph.BTNode.Index"));

		ChildSlot
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				// Add a dummy box here to make sure the widget doesnt get smaller than the brush
				SNew(SBox)
				.WidthOverride(IndexBrush->ImageSize.X)
				.HeightOverride(IndexBrush->ImageSize.Y)
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SBorder)
				.BorderImage(IndexBrush)
				.BorderBackgroundColor(this, &SConversationIndex::GetColor)
				.Padding(FMargin(4.0f, 0.0f, 4.0f, 1.0f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(InArgs._Text)
					.Font(FAppStyle::GetFontStyle("BTEditor.Graph.BTNode.IndexText"))
				]
			]
		];
	}

	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		OnHoverStateChangedEvent.ExecuteIfBound(true);
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		OnHoverStateChangedEvent.ExecuteIfBound(false);
		SCompoundWidget::OnMouseLeave(MouseEvent);
	}

	/** Get the color we use to display the rounded border */
	FSlateColor GetColor() const
	{
		if(OnGetIndexColorEvent.IsBound())
		{
			return OnGetIndexColorEvent.Execute(IsHovered());
		}

		return FSlateColor::UseForeground();
	}

private:
	/** Delegate event fired when the hover state of this widget changes */
	FOnHoverStateChanged OnHoverStateChangedEvent;

	/** Delegate used to receive the color of the node, depending on hover state and state of other siblings */
	FOnGetIndexColor OnGetIndexColorEvent;
};

/////////////////////////////////////////////////////
// SConversationGraphNode

void SConversationGraphNode::Construct(const FArguments& InArgs, UConversationGraphNode* InNode)
{
	DebuggerStateDuration = 0.0f;
	DebuggerStateCounter = INDEX_NONE;
	bSuppressDebuggerTriggers = false;

	SGraphNodeAI::Construct(SGraphNodeAI::FArguments(), InNode);
}

void SConversationGraphNode::AddSubNodeWidget(TSharedPtr<SGraphNode> NewSubNodeWidget, ESubNodeWidgetLocation Location)
{
	if (OwnerGraphPanelPtr.IsValid())
	{
		NewSubNodeWidget->SetOwner(OwnerGraphPanelPtr.Pin().ToSharedRef());
		OwnerGraphPanelPtr.Pin()->AttachGraphEvents(NewSubNodeWidget);
	}
	NewSubNodeWidget->UpdateGraphNode();


	FSubNodeWidgetStuff& SubNodeAreaInfo = ChildNodeStuff.FindChecked(Location);

	SubNodeAreaInfo.ChildNodeBox->AddSlot().AutoHeight()
	[
		NewSubNodeWidget.ToSharedRef()
	];

	SubNodeAreaInfo.ChildNodeWidgets.Add(NewSubNodeWidget);

	AddSubNode(NewSubNodeWidget);
}

FSlateColor SConversationGraphNode::GetBorderBackgroundColor() const
{
	UConversationGraphNode* ConversationGraphNode = Cast<UConversationGraphNode>(GraphNode);
	UConversationGraphNode* ParentConversationGraphNode = ConversationGraphNode ? Cast<UConversationGraphNode>(ConversationGraphNode->ParentNode) : nullptr;

	const bool bSelectedSubNode = ParentConversationGraphNode && GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode);
	
	if (bSelectedSubNode)
	{
		return ConversationEditorColors::NodeBorder::Selected;
	}

	//@TODO: CONVERSATION: Search highlighting
// 		if (BTGraphNode->bHighlightInSearchTree)
// 		{
// 			return ConversationEditorColors::NodeBorder::QuickFind;
// 		}

	// Grey out disconnected or commented out nodes
	if (!IsNodeReachable())
	{
		return ConversationEditorColors::NodeBorder::Disconnected;
	}

	return ConversationEditorColors::NodeBorder::Inactive;
}

FSlateColor SConversationGraphNode::GetBackgroundColor() const
{
	UConversationGraphNode* ConversationNode = CastChecked<UConversationGraphNode>(GraphNode);

	FLinearColor NodeColor = ConversationNode->GetNodeBodyTintColor();

	if (ConversationNode->HasErrors())
	{
		NodeColor = ConversationEditorColors::NodeBody::Error;
	}

	return (FlashAlpha > 0.0f) ? FMath::Lerp(NodeColor, FlashColor, FlashAlpha) : NodeColor;
}

void SConversationGraphNode::UpdateGraphNode()
{
	bDragMarkerVisible = false;
	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SubNodes.Reset();
	OutputPinBox.Reset();

	ChildNodeStuff.Reset();
	ChildNodeStuff.Add(ESubNodeWidgetLocation::Above).ChildNodeBox = SNew(SVerticalBox);
	ChildNodeStuff.Add(ESubNodeWidgetLocation::Below).ChildNodeBox = SNew(SVerticalBox);
	
	UConversationGraphNode* ConversationNode = Cast<UConversationGraphNode>(GraphNode);

	if (ConversationNode)
	{
		for (UAIGraphNode* TestNode : ConversationNode->SubNodes)
		{
			if (UConversationGraphNode_Requirement* RequirementNode = Cast<UConversationGraphNode_Requirement>(TestNode))
			{
				TSharedPtr<SGraphNode> NewNode = FNodeFactory::CreateNodeWidget(RequirementNode);
				AddSubNodeWidget(NewNode, ESubNodeWidgetLocation::Above);
			}
		}

		for (UAIGraphNode* TestNode : ConversationNode->SubNodes)
		{
			if (UConversationGraphNode_Choice* ChoiceNode = Cast<UConversationGraphNode_Choice>(TestNode))
			{
				TSharedPtr<SGraphNode> NewNode = FNodeFactory::CreateNodeWidget(ChoiceNode);
				AddSubNodeWidget(NewNode, ESubNodeWidgetLocation::Above);
			}
		}

		for (UAIGraphNode* TestNode : ConversationNode->SubNodes)
		{
			if (UConversationGraphNode_SideEffect* SideEffectNode = Cast<UConversationGraphNode_SideEffect>(TestNode))
			{
				TSharedPtr<SGraphNode> NewNode = FNodeFactory::CreateNodeWidget(SideEffectNode);
				AddSubNodeWidget(NewNode, ESubNodeWidgetLocation::Below);
			}
		}
	}

	TSharedPtr<SErrorText> ErrorText;
	TSharedPtr<STextBlock> DescriptionText; 
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	TWeakPtr<SNodeTitle> WeakNodeTitle = NodeTitle;
	auto GetNodeTitlePlaceholderWidth = [WeakNodeTitle]() -> FOptionalSize
	{
		TSharedPtr<SNodeTitle> NodeTitlePin = WeakNodeTitle.Pin();
		const float DesiredWidth = (NodeTitlePin.IsValid()) ? NodeTitlePin->GetTitleSize().X : 0.0f;
		return FMath::Max(75.0f, DesiredWidth);
	};
	auto GetNodeTitlePlaceholderHeight = [WeakNodeTitle]() -> FOptionalSize
	{
		TSharedPtr<SNodeTitle> NodeTitlePin = WeakNodeTitle.Pin();
		const float DesiredHeight = (NodeTitlePin.IsValid()) ? NodeTitlePin->GetTitleSize().Y : 0.0f;
		return FMath::Max(22.0f, DesiredHeight);
	};

	const bool bIsEmbeddedNode = ConversationNode && ConversationNode->IsSubNode();
	const FMargin NodePadding = bIsEmbeddedNode ? FMargin(2.0f) : FMargin(8.0f);

	if (bShowExecutionIndexInEditorMode)
	{
		IndexOverlay = SNew(SConversationIndex)
			.ToolTipText(this, &SConversationGraphNode::GetIndexTooltipText)
			.Visibility(this, &SConversationGraphNode::GetIndexVisibility)
			.Text(this, &SConversationGraphNode::GetIndexText)
			.OnHoverStateChanged(this, &SConversationGraphNode::OnIndexHoverStateChanged)
			.OnGetIndexColor(this, &SConversationGraphNode::GetIndexColor);
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FPropertyRowGeneratorArgs Args;
	Args.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	Args.NotifyHook = this;
	PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);

	if (UConversationGraphNode* TaskGraphNode = Cast<UConversationGraphNode>(GraphNode))
	{
		if (TaskGraphNode->NodeInstance)
		{
			PropertyRowGenerator->SetObjects({ TaskGraphNode->NodeInstance });
		}
	}

	PropertyRowGenerator->OnRowsRefreshed().AddSP(this, &SConversationGraphNode::PropertyRowsRefreshed);

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "Graph.StateNode.Body" ) )
			.Padding(0.0f)
			.BorderBackgroundColor( this, &SConversationGraphNode::GetBorderBackgroundColor )
			.OnMouseButtonDown(this, &SConversationGraphNode::OnMouseDown)
			[
				SNew(SOverlay)

				// Pins and node details
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					// INPUT PIN AREA
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.MinDesiredHeight(NodePadding.Top)
						[
							SAssignNew(LeftNodeBox, SVerticalBox)
						]
					]

					// STATE NAME AREA
					+SVerticalBox::Slot()
					.Padding(FMargin(NodePadding.Left, 0.0f, NodePadding.Right, 0.0f))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							ChildNodeStuff[ESubNodeWidgetLocation::Above].ChildNodeBox.ToSharedRef()
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(NodeBody, SBorder)
							.BorderImage( FAppStyle::GetBrush("BTEditor.Graph.BTNode.Body") )
							.BorderBackgroundColor( this, &SConversationGraphNode::GetBackgroundColor )
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Center)
							.Visibility(EVisibility::SelfHitTestInvisible)
							[
								SNew(SOverlay)
								+SOverlay::Slot()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								[
									SNew(SVerticalBox)
									+SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)
										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											// POPUP ERROR MESSAGE
											SAssignNew(ErrorText, SErrorText )
											.BackgroundColor( this, &SConversationGraphNode::GetErrorColor )
											.ToolTipText( this, &SConversationGraphNode::GetErrorMsgToolTip )
										]
										
										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SLevelOfDetailBranchNode)
											.UseLowDetailSlot(this, &SConversationGraphNode::UseLowDetailNodeTitles)
											.LowDetail()
											[
												SNew(SBox)
												.WidthOverride_Lambda(GetNodeTitlePlaceholderWidth)
												.HeightOverride_Lambda(GetNodeTitlePlaceholderHeight)
											]
											.HighDetail()
											[
												SNew(SHorizontalBox)
												+SHorizontalBox::Slot()
												.AutoWidth()
												.VAlign(VAlign_Center)
												[
													SNew(SImage)
													.Image(this, &SConversationGraphNode::GetNameIcon)
												]
												+SHorizontalBox::Slot()
												.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
												[
													SNew(SVerticalBox)
													+SVerticalBox::Slot()
													.AutoHeight()
													[
														SAssignNew(InlineEditableText, SInlineEditableTextBlock)
														.Style( FAppStyle::Get(), "Graph.StateNode.NodeTitleInlineEditableText" )
														.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
														.OnVerifyTextChanged(this, &SConversationGraphNode::OnVerifyNameTextChanged)
														.OnTextCommitted(this, &SConversationGraphNode::OnNameTextCommited)
														.IsReadOnly( this, &SConversationGraphNode::IsNameReadOnly )
														.IsSelected(this, &SConversationGraphNode::IsSelectedExclusively)
													]
													+SVerticalBox::Slot()
													.AutoHeight()
													[
														NodeTitle.ToSharedRef()
													]
												]
											]
										]
									]

									+SVerticalBox::Slot()
									.AutoHeight()
									[
										// TASK REQUIREMENT MESSAGE
										SNew(STextBlock)
										.Visibility(this, &SConversationGraphNode::GetTaskRequirementsVisibility)
										.Text(LOCTEXT("TaskHasRequirements", "[Task Has Requirements]"))
									]

									+SVerticalBox::Slot()
									.AutoHeight()
									[
										// TASK CHOICES MESSAGE
										SNew(STextBlock)
										.Visibility(this, &SConversationGraphNode::GetTaskGeneratesChoicesVisibility)
										.Text(LOCTEXT("TaskHasDynamicChoices", "[Task Generates Choices]"))
									]

									+SVerticalBox::Slot()
									.AutoHeight()
									[
										// DESCRIPTION MESSAGE
										SAssignNew(DescriptionText, STextBlock )
										.Visibility(this, &SConversationGraphNode::GetDescriptionVisibility)
										.Text(this, &SConversationGraphNode::GetDescription)
									]

									+SVerticalBox::Slot()
									.Expose(PropertyDetailsSlot)
									.AutoHeight()
									[
										SNullWidget::NullWidget
									]
								]
							]
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							ChildNodeStuff[ESubNodeWidgetLocation::Below].ChildNodeBox.ToSharedRef()
						]
					]

					// OUTPUT PIN AREA
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.MinDesiredHeight(NodePadding.Bottom)
						[
							SAssignNew(RightNodeBox, SVerticalBox)
							+SVerticalBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.Padding(20.0f,0.0f)
							.FillHeight(1.0f)
							[
								SAssignNew(OutputPinBox, SHorizontalBox)
							]
						]
					]
				]

				// Drag marker overlay
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SBorder)
					.BorderBackgroundColor(ConversationEditorColors::Action::DragMarker)
					.ColorAndOpacity(ConversationEditorColors::Action::DragMarker)
					.BorderImage(FAppStyle::GetBrush("BTEditor.Graph.BTNode.Body"))
					.Visibility(this, &SConversationGraphNode::GetDragOverMarkerVisibility)
					[
						SNew(SBox)
						.HeightOverride(4)
					]
				]

				// Blueprint indicator overlay
				+SOverlay::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("BTEditor.Graph.BTNode.Blueprint")))
					.Visibility(this, &SConversationGraphNode::GetBlueprintIconVisibility)
				]
			]
		];
	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered( this, &SGraphNode::IsHovered );

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	ErrorReporting = ErrorText;
	//ErrorReporting->SetError(TEXT("Testerror"));
	UpdateErrorInfo();

	PropertyRowsRefreshed();

	CreatePinWidgets();
}

void SConversationGraphNode::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SGraphNode::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );
	CachedPosition = FVector2D(AllottedGeometry.AbsolutePosition / AllottedGeometry.Scale);

	UConversationGraphNode* MyNode = Cast<UConversationGraphNode>(GraphNode);

	//@TODO: CONVERSATION: DEBUGGER
// 	if (MyNode && MyNode->DebuggerUpdateCounter != DebuggerStateCounter)
// 	{
// 		DebuggerStateCounter = MyNode->DebuggerUpdateCounter;
// 		DebuggerStateDuration = 0.0f;
// 		bSuppressDebuggerColor = false;
// 		bSuppressDebuggerTriggers = false;
// 	}

	DebuggerStateDuration += InDeltaTime;

	UConversationGraphNode* BTGraphNode = Cast<UConversationGraphNode>(GraphNode);
	float NewFlashAlpha = 0.0f;
	TriggerOffsets.Reset();

#if 0
	if (BTGraphNode && FConversationDebugger::IsPlaySessionPaused())
	{
		const float SearchPathDelay = 0.5f;
		const float SearchPathBlink = 1.0f;
		const float SearchPathBlinkFreq = 10.0f;
		const float SearchPathKeepTime = 2.0f;
		const float ActiveFlashDuration = 0.2f;

		const bool bHasResult = BTGraphNode->bDebuggerMarkSearchSucceeded || BTGraphNode->bDebuggerMarkSearchFailed;
		const bool bHasTriggers = !bSuppressDebuggerTriggers && (BTGraphNode->bDebuggerMarkSearchTrigger || BTGraphNode->bDebuggerMarkSearchFailedTrigger);
		if (bHasResult || bHasTriggers)
		{
			const float FlashStartTime = BTGraphNode->DebuggerSearchPathIndex * SearchPathDelay;
			const float FlashStopTime = (BTGraphNode->DebuggerSearchPathSize * SearchPathDelay) + SearchPathKeepTime;
			
			UConversationGraphNode_Decorator* BTGraph_Decorator = Cast<UConversationGraphNode_Decorator>(GraphNode);
			UConversationGraphNode_CompositeDecorator* BTGraph_CompDecorator = Cast<UConversationGraphNode_CompositeDecorator>(GraphNode);

			bSuppressDebuggerColor = (DebuggerStateDuration < FlashStopTime);
			if (bSuppressDebuggerColor)
			{
				if (bHasResult && (BTGraph_Decorator || BTGraph_CompDecorator))
				{
					NewFlashAlpha =
						(DebuggerStateDuration > FlashStartTime + SearchPathBlink) ? 1.0f :
						(FMath::TruncToInt(DebuggerStateDuration * SearchPathBlinkFreq) % 2) ? 1.0f : 0.0f;
				} 
			}

			FlashColor = BTGraphNode->bDebuggerMarkSearchSucceeded ?
				ConversationEditorColors::Debugger::SearchSucceeded :
				ConversationEditorColors::Debugger::SearchFailed;
		}
		else if (BTGraphNode->bDebuggerMarkFlashActive)
		{
			NewFlashAlpha = (DebuggerStateDuration < ActiveFlashDuration) ?
				FMath::Square(1.0f - (DebuggerStateDuration / ActiveFlashDuration)) : 
				0.0f;

			FlashColor = ConversationEditorColors::Debugger::TaskFlash;
		}

		if (bHasTriggers)
		{
			// find decorator that caused restart
			for (int32 i = 0; i < DecoratorWidgets.Num(); i++)
			{
				if (DecoratorWidgets[i].IsValid())
				{
					SConversationGraphNode* TestSNode = (SConversationGraphNode*)DecoratorWidgets[i].Get();
					UConversationGraphNode* ChildNode = Cast<UConversationGraphNode>(TestSNode->GraphNode);
					if (ChildNode && (ChildNode->bDebuggerMarkSearchFailedTrigger || ChildNode->bDebuggerMarkSearchTrigger))
					{
						TriggerOffsets.Add(FNodeBounds(TestSNode->GetCachedPosition() - CachedPosition, TestSNode->GetDesiredSize()));
					}
				}
			}

			// when it wasn't any of them, add node itself to triggers (e.g. parallel's main task)
			if (DecoratorWidgets.Num() == 0)
			{
				TriggerOffsets.Add(FNodeBounds(FVector2D(0,0),GetDesiredSize()));
			}
		}
	}
#endif
	FlashAlpha = NewFlashAlpha;
}

FReply SConversationGraphNode::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return SGraphNode::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent );
}

FText SConversationGraphNode::GetPinTooltip(UEdGraphPin* GraphPinObj) const
{
	FText HoverText = FText::GetEmpty();

	check(GraphPinObj != nullptr);
	UEdGraphNode* OwningGraphNode = GraphPinObj->GetOwningNode();
	if (OwningGraphNode != nullptr)
	{
		FString HoverStr;
		OwningGraphNode->GetPinHoverText(*GraphPinObj, /*out*/HoverStr);
		if (!HoverStr.IsEmpty())
		{
			HoverText = FText::FromString(HoverStr);
		}
	}

	return HoverText;
}

void SConversationGraphNode::CreatePinWidgets()
{
	UConversationGraphNode* StateNode = CastChecked<UConversationGraphNode>(GraphNode);

	for (int32 PinIdx = 0; PinIdx < StateNode->Pins.Num(); PinIdx++)
	{
		UEdGraphPin* MyPin = StateNode->Pins[PinIdx];
		if (!MyPin->bHidden)
		{
			TSharedPtr<SGraphPin> NewPin = SNew(SConversationPin, MyPin)
				.ToolTipText( this, &SConversationGraphNode::GetPinTooltip, MyPin);

			AddPin(NewPin.ToSharedRef());
		}
	}
}

void SConversationGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner( SharedThis(this) );

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = PinObj && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility( TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced) );
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(20.0f,0.0f)
			[
				PinToAdd
			];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		const bool bIsSingleTaskPin = PinObj && (PinObj->PinType.PinCategory == UConversationGraphTypes::PinCategory_SingleTask);
		if (bIsSingleTaskPin)
		{
			OutputPinBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(0.4f)
			.Padding(0,0,20.0f,0)
			[
				PinToAdd
			];
		}
		else
		{
			OutputPinBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(1.0f)
			[
				PinToAdd
			];
		}
		OutputPins.Add(PinToAdd);
	}
}

TSharedPtr<SToolTip> SConversationGraphNode::GetComplexTooltip()
{
	//@TODO: CONVERSATION: Better tooltips
// 	UConversationGraphNode_CompositeDecorator* DecoratorNode = Cast<UConversationGraphNode_CompositeDecorator>(GraphNode);
// 	if (DecoratorNode && DecoratorNode->GetBoundGraph())
// 	{
// 		return SNew(SToolTip)
// 			[
// 				SNew(SOverlay)
// 				+SOverlay::Slot()
// 				[
// 					// Create the tooltip graph preview, make sure to disable state overlays to
// 					// prevent the PIE / read-only borders from obscuring the graph
// 					SNew(SGraphPreviewer, DecoratorNode->GetBoundGraph())
// 					.CornerOverlayText(LOCTEXT("CompositeDecoratorOverlayText", "Composite Decorator"))
// 					.ShowGraphStateOverlay(false)
// 				]
// 				+SOverlay::Slot()
// 				.Padding(2.0f)
// 				[
// 					SNew(STextBlock)
// 					.Text(LOCTEXT("CompositeDecoratorTooltip", "Double-click to Open"))
// 					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
// 				]
// 			];
// 	}

// 	UConversationGraphNode_Task* TaskNode = Cast<UConversationGraphNode_Task>(GraphNode);
// 	if(TaskNode && TaskNode->NodeInstance)
// 	{
// 		UBTTask_RunBehavior* RunBehavior = Cast<UBTTask_RunBehavior>(TaskNode->NodeInstance);
// 		if(RunBehavior && RunBehavior->GetSubtreeAsset() && RunBehavior->GetSubtreeAsset()->BTGraph)
// 		{
// 			return SNew(SToolTip)
// 				[
// 					SNew(SOverlay)
// 					+SOverlay::Slot()
// 					[
// 						// Create the tooltip graph preview, make sure to disable state overlays to
// 						// prevent the PIE / read-only borders from obscuring the graph
// 						SNew(SGraphPreviewer, RunBehavior->GetSubtreeAsset()->BTGraph)
// 						.CornerOverlayText(LOCTEXT("RunBehaviorOverlayText", "Run Behavior"))
// 						.ShowGraphStateOverlay(false)
// 					]
// 					+SOverlay::Slot()
// 					.Padding(2.0f)
// 					[
// 						SNew(STextBlock)
// 						.Text(LOCTEXT("RunBehaviorTooltip", "Double-click to Open"))
// 						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
// 					]
// 				];
// 		}
// 	}

	return IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SGraphNode::GetNodeTooltip), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName());
}

const FSlateBrush* SConversationGraphNode::GetNameIcon() const
{	
	UConversationGraphNode* BTGraphNode = Cast<UConversationGraphNode>(GraphNode);
	return BTGraphNode != nullptr ? FAppStyle::GetBrush(BTGraphNode->GetNameIcon()) : FAppStyle::GetBrush(TEXT("BTEditor.Graph.BTNode.Icon"));
}

static UConversationGraphNode* GetParentNode(UEdGraphNode* GraphNode)
{
	UConversationGraphNode* BTGraphNode = Cast<UConversationGraphNode>(GraphNode);
	if (BTGraphNode->ParentNode != nullptr)
	{
		BTGraphNode = Cast<UConversationGraphNode>(BTGraphNode->ParentNode);
	}

	UEdGraphPin* MyInputPin = BTGraphNode->GetInputPin();
	UEdGraphPin* MyParentOutputPin = nullptr;
	if (MyInputPin != nullptr && MyInputPin->LinkedTo.Num() > 0)
	{
		MyParentOutputPin = MyInputPin->LinkedTo[0];
		if(MyParentOutputPin != nullptr)
		{
			if(MyParentOutputPin->GetOwningNode() != nullptr)
			{
				return CastChecked<UConversationGraphNode>(MyParentOutputPin->GetOwningNode());
			}
		}
	}

	return nullptr;
}

void SConversationGraphNode::OnIndexHoverStateChanged(bool bHovered)
{
//@TODO: CONVERSATION: Index highlighting code (here and next func)
// 	UConversationGraphNode* ParentNode = GetParentNode(GraphNode);
// 	if(ParentNode != nullptr)
// 	{
// 		ParentNode->bHighlightChildNodeIndices = bHovered;
// 	}
}

FSlateColor SConversationGraphNode::GetIndexColor(bool bHovered) const
{
	UConversationGraphNode* ParentNode = GetParentNode(GraphNode);
	const bool bHighlightHover = bHovered /*|| (ParentNode && ParentNode->bHighlightChildNodeIndices)*/; //@TODO: CONVERSATION: highlights?

	static const FName HoveredColor("BTEditor.Graph.BTNode.Index.HoveredColor");
	static const FName DefaultColor("BTEditor.Graph.BTNode.Index.Color");

	return bHighlightHover ? FAppStyle::Get().GetSlateColor(HoveredColor) : FAppStyle::Get().GetSlateColor(DefaultColor);
}

EVisibility SConversationGraphNode::GetIndexVisibility() const
{
	// always hide the index on unreachable nodes and entry points
	if (!IsNodeReachable() || GraphNode->IsA(UConversationGraphNode_EntryPoint::StaticClass()))
	{
		return EVisibility::Collapsed;
	}

	UConversationGraphNode* StateNode = CastChecked<UConversationGraphNode>(GraphNode);
	UEdGraphPin* MyInputPin = StateNode->GetInputPin();
	UEdGraphPin* MyParentOutputPin = NULL;
	if (MyInputPin != NULL && MyInputPin->LinkedTo.Num() > 0)
	{
		MyParentOutputPin = MyInputPin->LinkedTo[0];
	}

	// Visible if we are in PIE or if we have siblings
	CA_SUPPRESS(6235);
	const bool bCanShowIndex = (bShowExecutionIndexInEditorMode || GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != NULL) || (MyParentOutputPin && MyParentOutputPin->LinkedTo.Num() > 1);

	// LOD this out once things get too small
	TSharedPtr<SGraphPanel> MyOwnerPanel = GetOwnerPanel();
	return (bCanShowIndex && (!MyOwnerPanel.IsValid() || MyOwnerPanel->GetCurrentLOD() > EGraphRenderingLOD::LowDetail)) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SConversationGraphNode::GetIndexText() const
{
	UConversationGraphNode* StateNode = CastChecked<UConversationGraphNode>(GraphNode);
	UEdGraphPin* MyInputPin = StateNode->GetInputPin();
	UEdGraphPin* MyParentOutputPin = NULL;
	if (MyInputPin != NULL && MyInputPin->LinkedTo.Num() > 0)
	{
		MyParentOutputPin = MyInputPin->LinkedTo[0];
	}

	int32 Index = 0;

	CA_SUPPRESS(6235);
	if (bShowExecutionIndexInEditorMode || GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr)
	{
		// show execution index (debugging purposes)
// 		UBTNode* BTNode = Cast<UBTNode>(StateNode->NodeInstance);
// 		Index = (BTNode && BTNode->GetExecutionIndex() < 0xffff) ? BTNode->GetExecutionIndex() : -1;
		Index = -1;
	}
	else
	{
		// show child index
		if (MyParentOutputPin != NULL)
		{
			for (Index = 0; Index < MyParentOutputPin->LinkedTo.Num(); ++Index)
			{
				if (MyParentOutputPin->LinkedTo[Index] == MyInputPin)
				{
					break;
				}
			}
		}
	}

	return FText::AsNumber(Index);
}

FText SConversationGraphNode::GetIndexTooltipText() const
{
	CA_SUPPRESS(6235);
	if (bShowExecutionIndexInEditorMode || GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != NULL)
	{
		return LOCTEXT("ExecutionIndexTooltip", "Execution index: this shows the order in which nodes are executed.");
	}
	else
	{
		return LOCTEXT("ChildIndexTooltip", "Child index: this shows the order in which child nodes are executed.");
	}
}

EVisibility SConversationGraphNode::GetBlueprintIconVisibility() const
{
	UConversationGraphNode* BTGraphNode = Cast<UConversationGraphNode>(GraphNode);
	const bool bCanShowIcon = (BTGraphNode != nullptr && BTGraphNode->UsesBlueprint());

	// LOD this out once things get too small
	TSharedPtr<SGraphPanel> MyOwnerPanel = GetOwnerPanel();
	return (bCanShowIcon && (!MyOwnerPanel.IsValid() || MyOwnerPanel->GetCurrentLOD() > EGraphRenderingLOD::LowDetail)) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SConversationGraphNode::GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
{
	UConversationGraphNode* ConversationGraphNode = CastChecked<UConversationGraphNode>(GraphNode);

	//@TODO: CONVERSATION: Debugger
// 	if (ConversationGraphNode->bHasBreakpoint)
// 	{
// 		FOverlayBrushInfo BreakpointOverlayInfo;
// 		BreakpointOverlayInfo.Brush = BTNode->bIsBreakpointEnabled ?
// 			FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.Breakpoint.Enabled")) :
// 			FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.Breakpoint.Disabled"));
// 
// 		if (BreakpointOverlayInfo.Brush)
// 		{
// 			BreakpointOverlayInfo.OverlayOffset -= BreakpointOverlayInfo.Brush->ImageSize / 2.f;
// 		}
// 
// 		Brushes.Add(BreakpointOverlayInfo);
// 	}
// 
// 	if (FConversationDebugger::IsPlaySessionPaused())
// 	{
// 		if (BTNode->bDebuggerMarkBreakpointTrigger || (BTNode->bDebuggerMarkCurrentlyActive && BTNode->IsA(UConversationGraphNode_Task::StaticClass())))
// 		{
// 			FOverlayBrushInfo IPOverlayInfo;
// 
// 			IPOverlayInfo.Brush = BTNode->bDebuggerMarkBreakpointTrigger ? FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.BreakOnBreakpointPointer")) : 
// 				FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.ActiveNodePointer"));
// 			if (IPOverlayInfo.Brush)
// 			{
// 				float Overlap = 10.f;
// 				IPOverlayInfo.OverlayOffset.X = (WidgetSize.X/2.f) - (IPOverlayInfo.Brush->ImageSize.X/2.f);
// 				IPOverlayInfo.OverlayOffset.Y = (Overlap - IPOverlayInfo.Brush->ImageSize.Y);
// 			}
// 
// 			IPOverlayInfo.AnimationEnvelope = FVector2D(0.f, 10.f);
// 			Brushes.Add(IPOverlayInfo);
// 		}
// 
// 		if (TriggerOffsets.Num())
// 		{
// 			FOverlayBrushInfo IPOverlayInfo;
// 
// 			IPOverlayInfo.Brush = FAppStyle::GetBrush(BTNode->bDebuggerMarkSearchTrigger ?
// 				TEXT("BTEditor.DebuggerOverlay.SearchTriggerPointer") :
// 				TEXT("BTEditor.DebuggerOverlay.FailedTriggerPointer") );
// 
// 			if (IPOverlayInfo.Brush)
// 			{
// 				for (int32 i = 0; i < TriggerOffsets.Num(); i++)
// 				{
// 					IPOverlayInfo.OverlayOffset.X = -IPOverlayInfo.Brush->ImageSize.X;
// 					IPOverlayInfo.OverlayOffset.Y = TriggerOffsets[i].Position.Y + TriggerOffsets[i].Size.Y / 2 - IPOverlayInfo.Brush->ImageSize.Y / 2;
// 
// 					IPOverlayInfo.AnimationEnvelope = FVector2D(10.f, 0.f);
// 					Brushes.Add(IPOverlayInfo);
// 				}
// 			}
// 		}
// 	}
}

TArray<FOverlayWidgetInfo> SConversationGraphNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets;

	check(NodeBody.IsValid());

	FVector2f Origin(0.0f, 0.0f);

	// build overlays for sub-nodes (above)
	for (const TSharedPtr<SGraphNode>& ChildWidget : ChildNodeStuff.FindChecked(ESubNodeWidgetLocation::Above).ChildNodeWidgets)
	{
		TArray<FOverlayWidgetInfo> OverlayWidgets = ChildWidget->GetOverlayWidgets(bSelected, WidgetSize);
		for(auto& OverlayWidget : OverlayWidgets)
		{
			OverlayWidget.OverlayOffset.Y += Origin.Y;
		}
		Widgets.Append(OverlayWidgets);
		Origin.Y += ChildWidget->GetDesiredSize().Y;
	}

	if (IndexOverlay.IsValid())
	{
		FOverlayWidgetInfo Overlay(IndexOverlay);
		Overlay.OverlayOffset = FVector2f(WidgetSize.X - (IndexOverlay->GetDesiredSize().X * 0.5f), Origin.Y);
		Widgets.Add(Overlay);
	}

	Origin.Y += NodeBody->GetDesiredSize().Y;

	// build overlays for sub-nodes (below)
	for (const TSharedPtr<SGraphNode>& ChildWidget : ChildNodeStuff.FindChecked(ESubNodeWidgetLocation::Below).ChildNodeWidgets)
	{
		TArray<FOverlayWidgetInfo> OverlayWidgets = ChildWidget->GetOverlayWidgets(bSelected, WidgetSize);
		for (auto& OverlayWidget : OverlayWidgets)
		{
			OverlayWidget.OverlayOffset.Y += Origin.Y;
		}
		Widgets.Append(OverlayWidgets);
		Origin.Y += ChildWidget->GetDesiredSize().Y;
	}

	return Widgets;
}

TSharedRef<SGraphNode> SConversationGraphNode::GetNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<SGraphNode> SubNode = GetSubNodeUnderCursor(MyGeometry, MouseEvent);
	return SubNode.IsValid() ? SubNode.ToSharedRef() : StaticCastSharedRef<SGraphNode>(AsShared());
}

void SConversationGraphNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNodeAI::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	UConversationGraphNode* ConversationGraphNode = CastChecked<UConversationGraphNode>(GraphNode);
	if (!ConversationGraphNode->IsSubNode())
	{
		//@TODO: CONVERSATION: Reordering dialog choices based on X position
// 		if (UConversationGraph* ConversationGraph = ConversationGraphNode->GetConversationGraph())
// 		{
// 			for (int32 Idx = 0; Idx < BTGraphNode->Pins.Num(); Idx++)
// 			{
// 				UEdGraphPin* Pin = BTGraphNode->Pins[Idx];
// 				if (Pin && Pin->Direction == EGPD_Input && Pin->LinkedTo.Num() == 1) 
// 				{
// 					UEdGraphPin* ParentPin = Pin->LinkedTo[0];
// 					if (ParentPin)
// 					{
// 						BTGraph->RebuildChildOrder(ParentPin->GetOwningNode());
// 					}
// 				}
// 			}
// 		}
	}
}

FReply SConversationGraphNode::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FReply Reply = SGraphNodeAI::OnDrop(MyGeometry, DragDropEvent);

	if (ConversationEditorCVar::SelectWholeNodeOnDropCVar)
	{
		GetOwnerPanel()->SelectionManager.SelectSingleNode(GraphNode);
	}

	return Reply;
}

bool SConversationGraphNode::IsNodeReachable() const
{
	UConversationGraphNode* ConversationGraphNode = CastChecked<UConversationGraphNode>(GraphNode);
	UConversationGraphNode* ParentConversationGraphNode = Cast<UConversationGraphNode>(ConversationGraphNode->ParentNode);
	UConversationGraphNode* TopLevelGraphNode = ParentConversationGraphNode ? ParentConversationGraphNode : ConversationGraphNode;

	UConversationDatabase* ConversationAsset = Cast<UConversationDatabase>(ConversationGraphNode->GetGraph()->GetOuter());
	return ConversationAsset && ConversationAsset->IsNodeReachable(TopLevelGraphNode->NodeGuid);
}

EVisibility SConversationGraphNode::GetTaskRequirementsVisibility() const
{
	if (UConversationGraphNode_Task* TaskGraphNode = Cast<UConversationGraphNode_Task>(GraphNode))
	{
		if (UConversationTaskNode* TaskNode = Cast<UConversationTaskNode>(TaskGraphNode->NodeInstance))
		{
			if (TaskNode->bHasRequirements)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility SConversationGraphNode::GetTaskGeneratesChoicesVisibility() const
{
	if (UConversationGraphNode_Task* TaskGraphNode = Cast<UConversationGraphNode_Task>(GraphNode))
	{
		if (UConversationTaskNode* TaskNode = Cast<UConversationTaskNode>(TaskGraphNode->NodeInstance))
		{
			if (TaskNode->bHasDynamicChoices)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

void SConversationGraphNode::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{
//this->UpdateGraphNode()
//	const UEdGraphSchema* Schema = GetSchema();
//	if (Schema != nullptr)
//	{
//		Schema->ForceVisualizationCacheClear();
//	}
}

void SConversationGraphNode::PropertyRowsRefreshed()
{
	if (UConversationGraphNode* ConvGraphNode = Cast<UConversationGraphNode>(GraphNode))
	{
		if (UConversationNode* ConvNode = Cast<UConversationNode>(ConvGraphNode->NodeInstance))
		{
			if (!ConvNode->ShowPropertyEditors())
			{
				return;
			}
		}
	}

	TSharedPtr<SWidget> TimeWidget = nullptr;
	TSharedPtr<SWidget> ValueWidget = nullptr;

	TSharedRef<SGridPanel> GridPanel = 
		SNew(SGridPanel)
		.FillColumn(0, 0.0f)
		.FillColumn(1, 1.0f);

	static const FName NAME_ExposeOnSpawn(TEXT("ExposeOnSpawn"));

	int32 GridRow = 0;
	for (TSharedRef<IDetailTreeNode> RootNode : PropertyRowGenerator->GetRootTreeNodes())
	{
		TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
		RootNode->GetChildren(ChildNodes);

		for (TSharedRef<IDetailTreeNode> ChildNode : ChildNodes)
		{
			if (ChildNode->GetNodeType() == EDetailNodeType::Category)
			{
				continue;
			}

			TSharedPtr<IPropertyHandle> ChildNodePH = ChildNode->CreatePropertyHandle();

			if (!ChildNodePH || !ChildNodePH->HasMetaData(NAME_ExposeOnSpawn))
			{
				continue;
			}

			//TArray<TSharedRef<IDetailTreeNode>> SubChildren;
			//ChildNode->GetChildren(SubChildren);
			
			FNodeWidgets NodeWidgets = ChildNode->CreateNodeWidgets();
			if (NodeWidgets.NameWidget && NodeWidgets.ValueWidget)
			{
				GridPanel->AddSlot(0, GridRow)
				[
					NodeWidgets.NameWidget.ToSharedRef()
				];

				GridPanel->AddSlot(1, GridRow)
				[
					NodeWidgets.ValueWidget.ToSharedRef()
				];

				GridRow++;
			}
		}
	}

	(*PropertyDetailsSlot)
	[
		GridPanel
	];
}

#undef LOCTEXT_NAMESPACE
