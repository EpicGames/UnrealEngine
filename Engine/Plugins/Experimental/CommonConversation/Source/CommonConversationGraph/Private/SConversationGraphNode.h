// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNodeAI.h"
#include "Misc/NotifyHook.h"
#include "Widgets/SBoxPanel.h"

class SBorder;
struct FNodeBounds;
struct FOverlayBrushInfo;
struct FOverlayWidgetInfo;

class SHorizontalBox;
class SToolTip;
class SVerticalBox;
class UConversationGraphNode;
class SWidget;

enum class ESubNodeWidgetLocation : uint8
{
	Above,
	Below
};

struct FSubNodeWidgetStuff
{
	TSharedPtr<SVerticalBox> ChildNodeBox;

	TArray< TSharedPtr<SGraphNode> > ChildNodeWidgets;
};

//////////////////////////////////////////////////////////////////////

class SConversationGraphNode : public SGraphNodeAI, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SConversationGraphNode){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UConversationGraphNode* InNode);

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void CreatePinWidgets() override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	virtual TSharedRef<SGraphNode> GetNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SGraphNode interface

	/** handle double click */
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	
	/** adds a sub node widget inside current node */
	void AddSubNodeWidget(TSharedPtr<SGraphNode> NewSubNodeWidget, ESubNodeWidgetLocation Location);

	FVector2D GetCachedPosition() const { return CachedPosition; }

protected:
	uint32 bSuppressDebuggerColor : 1;
	uint32 bSuppressDebuggerTriggers : 1;
	
	/** time spent in current state */
	float DebuggerStateDuration;

	/** currently displayed state */
	int32 DebuggerStateCounter;

	/** debugger colors */
	FLinearColor FlashColor;
	float FlashAlpha;

	/** height offsets for search triggers */
	TArray<FNodeBounds> TriggerOffsets;

	/** cached draw position */
	FVector2D CachedPosition;

	TMap<ESubNodeWidgetLocation, FSubNodeWidgetStuff> ChildNodeStuff;

	TSharedPtr<SHorizontalBox> OutputPinBox;

	/** The widget we use to display the index of the node */
	TSharedPtr<SWidget> IndexOverlay;

	/** The node body widget, cached here so we can determine its size when we want ot position our overlays */
	TSharedPtr<SBorder> NodeBody;

	FSlateColor GetBorderBackgroundColor() const;
	FSlateColor GetBackgroundColor() const;

	virtual const FSlateBrush* GetNameIcon() const override;
	virtual EVisibility GetBlueprintIconVisibility() const;

	/** Get the visibility of the index overlay */
	EVisibility GetIndexVisibility() const;

	/** Get the text to display in the index overlay */
	FText GetIndexText() const;

	/** Get the tooltip for the index overlay */
	FText GetIndexTooltipText() const;

	/** Get the color to display for the index overlay. This changes on hover state of sibling nodes */
	FSlateColor GetIndexColor(bool bHovered) const;

	/** Handle hover state changing for the index widget - we use this to highlight sibling nodes */
	void OnIndexHoverStateChanged(bool bHovered);

	FText GetPinTooltip(UEdGraphPin* GraphPinObj) const;

	bool IsNodeReachable() const;

	EVisibility GetTaskRequirementsVisibility() const;
	EVisibility GetTaskGeneratesChoicesVisibility() const;

	void PropertyRowsRefreshed();

	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

	TSharedPtr<class IPropertyRowGenerator> PropertyRowGenerator;

	class SVerticalBox::FSlot* PropertyDetailsSlot;
};
