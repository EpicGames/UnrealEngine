// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

class FObjectPreSaveContext;
class UObject;
class UPackage;

COMMONCONVERSATIONGRAPH_API DECLARE_LOG_CATEGORY_EXTERN(LogCommonConversationGraph, Display, All);

class FCommonConversationGraphModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void HandlePreSavePackage(UPackage* Package, FObjectPreSaveContext ObjectSaveContext);
	void HandleBeginPIE(bool bIsSimulating);
};
