// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

class FName;
class FString;
template <typename FuncType> class TFunctionRef;

class UConversationGraphNode;
class UConversationGraph;
class UConversationDatabase;
class UEdGraphPin;

class FConversationCompiler
{
public:
	static UE_API int32 GetNumGraphs(UConversationDatabase* ConversationAsset);
	static UE_API UConversationGraph* GetGraphFromBank(UConversationDatabase* ConversationAsset, int32 Index);
	static UE_API void RebuildBank(UConversationDatabase* ConversationAsset);
	static UE_API UConversationGraph* AddNewGraph(UConversationDatabase* ConversationAsset, const FString& DesiredName);

	static UE_API int32 GetCompilerVersion();

	static UE_API void ScanAndRecompileOutOfDateCompiledConversations();

private:
	FConversationCompiler() {}

	// Creates a new graph but does not add it
	static UE_API UConversationGraph* CreateNewGraph(UConversationDatabase* ConversationAsset, FName GraphName);

	// Skips over knots.
	static UE_API void ForeachConnectedOutgoingConversationNode(UEdGraphPin* Pin, TFunctionRef<void(UConversationGraphNode*)> Predicate);

	/**
	 * Functions similarly to ForeachConnectedOutgoingConversationNode but visits nodes based on their left to right order
	 * 
	 * Knots are visited in depth-first order such that a graph like the following results in the order A B C D
	 * 
	 *            *
	 *           /|\
	 *          / | \
	 *         *  C  D
	 *        / \
	 *       /   \
	 *      /     \
	 *     /       \
	 *    A         B
	 */
	static UE_API void ForeachConnectedOutgoingConversationNodeSorted(UEdGraphPin* Pin, TFunctionRef<void(UConversationGraphNode*)> Predicate);
};

#undef UE_API
