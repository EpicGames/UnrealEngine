// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"

namespace ConversationEditorColors
{
	namespace NodeBody
	{
		const FLinearColor Task(0.24f, 0.055f, 0.715f);

		const FLinearColor RequirementColor(0.0f, 0.07f, 0.4f);
		const FLinearColor SideEffectColor(0.0f, 0.27f, 0.4f);
		const FLinearColor ChoiceColor(0.35f, 0.23f, 0.15f);

		const FLinearColor Default(0.15f, 0.15f, 0.15f);
		const FLinearColor EntryPoint(0.5f, 0.5f, 0.5f, 0.1f);
		const FLinearColor Error(1.0f, 0.0f, 0.0f);
	}

	namespace NodeBorder
	{
		const FLinearColor Inactive(0.08f, 0.08f, 0.08f);
		const FLinearColor Root(0.2f, 0.2f, 0.2f, 0.2f);
		const FLinearColor Selected(1.00f, 0.58f, 0.08f);
		const FLinearColor ActiveDebugging(1.0f, 1.0f, 0.0f);
		const FLinearColor InactiveDebugging(0.4f, 0.4f, 0.0f);
		const FLinearColor HighlightAbortRange0(0.0f, 0.22f, 0.4f);
		const FLinearColor HighlightAbortRange1(0.0f, 0.4f, 0.22f);
		const FLinearColor Disconnected(0.f, 0.f, 0.f);
		const FLinearColor BrokenWithParent(1.f, 0.f, 1.f);
		const FLinearColor QuickFind(0.f, 0.8f, 0.f);
	}

	namespace Pin
	{
		const FLinearColor Diff(0.9f, 0.2f, 0.15f);
		const FLinearColor Hover(1.0f, 0.7f, 0.0f);
		const FLinearColor Default(0.02f, 0.02f, 0.02f);
		const FLinearColor TaskOnly(0.13f, 0.03f, 0.4f);
		const FLinearColor CompositeOnly(0.02f, 0.02f, 0.02f);
		const FLinearColor SingleNode(0.02f, 0.02f, 0.02f);
	}
	
	namespace Connection
	{
		const FLinearColor Default(1.0f, 1.0f, 1.0f);
	}

	namespace Debugger
	{
		const FLinearColor TaskFlash(1.0f, 1.0f, 0.0f);

		const FLinearColor DescHeader(0.3f, 0.8f, 0.4f);
		const FLinearColor DescKeys(0.3f, 0.4f, 0.8f);
	}

	namespace Action
	{
		const FLinearColor DragMarker(1.0f, 1.0f, 0.2f);
	}
}
