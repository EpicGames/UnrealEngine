// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraph.h"
#include "ConversationGraph.generated.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

class UConversationDatabase;

UCLASS(MinimalAPI)
class UConversationGraph : public UAIGraph
{
	GENERATED_UCLASS_BODY()

public:
	// UAIGraph interface
	UE_API virtual void UpdateAsset(int32 UpdateFlags) override;
	// End of UAIGraph interface
};

#undef UE_API
