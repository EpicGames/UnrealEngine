// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"

class FSlateWindowElementList;
class UEdGraph;

class FConversationGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	UEdGraph* GraphObj;
	TMap<UEdGraphNode*, int32> NodeWidgetMap;

public:
	FConversationGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	// FConnectionDrawingPolicy interface 
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& PinGeometries, FArrangedChildren& ArrangedNodes) override;
	virtual void DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params) override;
	virtual void DrawSplineWithArrow(const FVector2f& StartPoint, const FVector2f& EndPoint, const FConnectionParams& Params) override;
	virtual void DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin) override;
	virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override;
	// End of FConnectionDrawingPolicy interface

protected:
	void Internal_DrawLineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params);
};
