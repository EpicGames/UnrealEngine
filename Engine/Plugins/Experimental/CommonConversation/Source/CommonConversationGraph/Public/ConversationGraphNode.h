// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphNode.h"
#include "ConversationGraphNode.generated.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

namespace ENodeTitleType { enum Type : int; }

class UConversationGraphNode_Knot;

UCLASS(MinimalAPI)
class UConversationGraphNode : public UAIGraphNode
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UEdGraphNode Interface
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
	UE_API virtual void FindDiffs(UEdGraphNode* OtherNode, FDiffResults& Results) override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeBodyTintColor() const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual bool CanJumpToDefinition() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode Interface

	UE_API virtual FText GetDescription() const override;

	/** check if node can accept breakpoints */
	virtual bool CanPlaceBreakpoints() const { return false; }

	/** gets icon resource name for title bar */
	UE_API virtual FName GetNameIcon() const;

	UE_API bool IsOutBoundConnectionAllowed(const UConversationGraphNode* OtherNode, FText& OutErrorMessage) const;
	UE_API bool IsOutBoundConnectionAllowed(const UConversationGraphNode_Knot* KnotNode, FText& OutErrorMessage) const;

	template<class T>
	T* GetRuntimeNode() const
	{
		return Cast<T>(NodeInstance);
	}

protected:
	UE_API void RequestRebuildConversation();
};

#undef UE_API
