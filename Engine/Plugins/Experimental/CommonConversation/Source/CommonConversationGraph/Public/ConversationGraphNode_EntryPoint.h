// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationGraphNode.h"
#include "ConversationGraphNode_EntryPoint.generated.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

/** Root node of this conversation block */
UCLASS(MinimalAPI)
class UConversationGraphNode_EntryPoint : public UConversationGraphNode
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FName GetNameIcon() const override;
};

#undef UE_API
