// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "ConversationGraphNode_Knot.generated.h"

class SGraphNode;
class UConversationGraphNode;

UCLASS(MinimalAPI)
class UConversationGraphNode_Knot : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override;
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual bool IsCompilerRelevant() const override { return false; }
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override { OutInputPinIndex = 0; OutOutputPinIndex = 1; return true; }
	// End of UEdGraphNode interface
	
	UEdGraphPin* GetInputPin() const
	{
		return Pins[0];
	}

	UEdGraphPin* GetOutputPin() const
	{
		return Pins[1];
	}

	void GatherAllInBoundGraphNodes(TArray<UConversationGraphNode*>& OutGraphNodes) const;
	void GatherAllOutBoundGraphNodes(TArray<UConversationGraphNode*>& OutGraphNodes) const;

	bool IsOutBoundConnectionAllowed(const UConversationGraphNode* OtherNode, FText& OutErrorMessage) const;
	bool IsOutBoundConnectionAllowed(const UConversationGraphNode_Knot* OtherKnotNode, FText& OutErrorMessage) const;

private:
	void GatherAllInBoundGraphNodes_Internal(TArray<UConversationGraphNode*>& OutGraphNodes, TArray<const UConversationGraphNode_Knot*>& VisitedKnots) const;
	void GatherAllOutBoundGraphNodes_Internal(TArray<UConversationGraphNode*>& OutGraphNodes, TArray<const UConversationGraphNode_Knot*>& VisitedKnots) const;
};
