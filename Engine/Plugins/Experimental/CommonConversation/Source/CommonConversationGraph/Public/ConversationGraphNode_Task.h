// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationGraphNode.h"
#include "ConversationGraphNode_Task.generated.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

class UGraphNodeContextMenuContext;
class UToolMenu;

class UEdGraph;
class UEdGraphSchema;

class FMenuBuilder;

UCLASS(MinimalAPI)
class UConversationGraphNode_Task : public UConversationGraphNode
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeBodyTintColor() const override;

	/** Gets a list of actions that can be done to this particular node */
	UE_API virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual void ApplyAddRequirementMenu(FMenuBuilder& MenuBuilder, UEdGraph* Graph);

	virtual bool CanPlaceBreakpoints() const override { return true; }
};

#undef UE_API
