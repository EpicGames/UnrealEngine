// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphSchema.h"

#include "ConversationGraphSchema.generated.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

class FSlateRect;
class UEdGraph;

class UConversationNode;
class UConversationGraphNode;

class FMenuBuilder;
class FConnectionDrawingPolicy;
class FSlateWindowElementList;

//////////////////////////////////////////////////////////////////////

/** Action to auto arrange the graph */
USTRUCT()
struct FConversationGraphSchemaAction_AutoArrange : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	FConversationGraphSchemaAction_AutoArrange()
		: FEdGraphSchemaAction() {}

	FConversationGraphSchemaAction_AutoArrange(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};


//////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class UConversationGraphSchema : public UAIGraphSchema
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin EdGraphSchema Interface
	UE_API virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	UE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	UE_API virtual const FPinConnectionResponse CanMergeNodes(const UEdGraphNode* A, const UEdGraphNode* B) const override;
	UE_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	UE_API virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	UE_API virtual int32 GetCurrentVisualizationCacheID() const override;
	UE_API virtual void ForceVisualizationCacheClear() const override;
	//~ End EdGraphSchema Interface

	UE_API virtual void GetGraphNodeContextActions(FGraphContextMenuBuilder& ContextMenuBuilder, int32 SubNodeFlags) const override;
	UE_API bool HasSubNodeClasses(int32 SubNodeFlags) const;
	UE_API virtual void GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const override;

protected:
	UE_API void AddConversationNodeOptions(const FString& CategoryName, FGraphContextMenuBuilder& ContextMenuBuilder, TSubclassOf<UConversationNode> RuntimeNodeType, TSubclassOf<UConversationGraphNode> EditorNodeType) const;

private:
	// ID for checking dirty status of node titles against, increases whenever 
	static UE_API int32 CurrentCacheRefreshID;
};

#undef UE_API
