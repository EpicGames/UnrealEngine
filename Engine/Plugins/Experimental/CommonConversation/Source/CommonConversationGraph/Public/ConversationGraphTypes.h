// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphPin.h"
#include "ConversationGraphTypes.generated.h"

#define UE_API COMMONCONVERSATIONGRAPH_API

struct FCompareNodeXLocation
{
	inline bool operator()(const UEdGraphPin& A, const UEdGraphPin& B) const
	{
		const UEdGraphNode* NodeA = A.GetOwningNode();
		const UEdGraphNode* NodeB = B.GetOwningNode();

		if (NodeA->NodePosX == NodeB->NodePosX)
		{
			return NodeA->NodePosY < NodeB->NodePosY;
		}

		return NodeA->NodePosX < NodeB->NodePosX;
	}
};

enum class EConversationGraphSubNodeType : uint8
{
	Requirement,
	SideEffect,
	Choice
};

struct FNodeBounds
{
	FVector2D Position;
	FVector2D Size;

	FNodeBounds(FVector2D InPos, FVector2D InSize)
	{
		Position = InPos;
		Size = InSize;
	}
};

UCLASS(MinimalAPI)
class UConversationGraphTypes : public UObject
{
	GENERATED_UCLASS_BODY()

	static UE_API const FName PinCategory_MultipleNodes;
	static UE_API const FName PinCategory_SingleComposite;
	static UE_API const FName PinCategory_SingleTask;
	static UE_API const FName PinCategory_SingleNode;
};

#undef UE_API
