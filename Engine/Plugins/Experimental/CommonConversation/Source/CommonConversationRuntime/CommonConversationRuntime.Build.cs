// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonConversationRuntime : ModuleRules
{
	public CommonConversationRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"GameplayTags",
				"DeveloperSettings"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Slate",
				"SlateCore",
				"Projects",
				"AIModule",
				"NetCore",
				"GameFeatures"
				// ... add private dependencies that you statically link with here ...	
			}
		);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}
