// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonConversationRuntimeModule.h"
#include "GameplayTagsManager.h"
#include "Interfaces/IPluginManager.h"
#include "CommonConversationRuntimeLogging.h"
#include "Modules/ModuleManager.h"
#include "UObject/CoreRedirects.h"

DEFINE_LOG_CATEGORY(LogCommonConversationRuntime);

#define LOCTEXT_NAMESPACE "CommonConversationEditor"

void FCommonConversationRuntimeModule::StartupModule()
{
	TSharedPtr<IPlugin> ThisPlugin = IPluginManager::Get().FindPlugin(TEXT("CommonConversation"));
	check(ThisPlugin.IsValid());
	
	UGameplayTagsManager::Get().AddTagIniSearchPath(ThisPlugin->GetBaseDir() / TEXT("Config") / TEXT("Tags"));


	TArray<FCoreRedirect> Redirects;
	Redirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/Script/CommonDialogueRuntime"), TEXT("/Script/CommonConversationRuntime"));
	FCoreRedirects::AddRedirectList(Redirects, TEXT("CommonConversationRuntime"));
}

void FCommonConversationRuntimeModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCommonConversationRuntimeModule, CommonConversationRuntime)
