// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationChoiceNode.h"
#include "ConversationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationChoiceNode)

bool UConversationChoiceNode::GenerateChoice(const FConversationContext& Context, FClientConversationOptionEntry& ChoiceEntry) const
{
	FillChoice(Context, ChoiceEntry);
	return true;
}

void UConversationChoiceNode::FillChoice_Implementation(const FConversationContext& Context, FClientConversationOptionEntry& ChoiceEntry) const
{
	ChoiceEntry.ChoiceText = DefaultChoiceDisplayText;
	ChoiceEntry.ChoiceTags = ChoiceTags;
}

void UConversationChoiceNode::NotifyChoicePickedByUser(const FConversationContext& InContext, const FClientConversationOptionEntry& InClientChoice) const
{

}

