// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationContext.h"
#include "ConversationInstance.h"
#include "Misc/RuntimeErrors.h"
#include "ConversationRegistry.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationContext)

#define LOCTEXT_NAMESPACE "ConversationGraph"

FConversationContext FConversationContext::CreateServerContext(UConversationInstance* InActiveConversation, const UConversationTaskNode* InTaskBeingConsidered)
{
	UWorld* World = InActiveConversation->GetWorld();

	FConversationContext Context;
	Context.ActiveConversation = InActiveConversation;
	Context.TaskBeingConsidered = InTaskBeingConsidered;
	Context.bServer_PRIVATE = true;
	Context.bClient_PRIVATE = World->GetNetMode() != NM_DedicatedServer;
	Context.ConversationRegistry = UConversationRegistry::GetFromWorld(World);

	return Context;
}

FConversationContext FConversationContext::CreateClientContext(UConversationParticipantComponent* InParticipantComponent, const UConversationTaskNode* InTaskBeingConsidered)
{
	UWorld* World = InParticipantComponent->GetWorld();

	FConversationContext Context;
	Context.ClientParticipant = InParticipantComponent;
	Context.TaskBeingConsidered = InTaskBeingConsidered;
	Context.bServer_PRIVATE = false;
	Context.bClient_PRIVATE = true;
	Context.ConversationRegistry = UConversationRegistry::GetFromWorld(World);

	return Context;
}

FConversationContext FConversationContext::CreateChildContext(const UConversationTaskNode* NewTaskBeingConsidered) const
{
	FConversationContext Context = *this;
	Context.TaskBeingConsidered = NewTaskBeingConsidered;
	
	return Context;
}

FConversationContext FConversationContext::CreateReturnScopeContext(const FConversationNodeHandle& NewReturnScope) const
{
	FConversationContext Context = *this;
	Context.ReturnScopeStack.Add(NewReturnScope);

	return Context;
}

UWorld* FConversationContext::GetWorld() const
{
	if (IsServerContext())
	{
		if (ActiveConversation)
		{
			return ActiveConversation->GetWorld();
		}
	}
	else
	{
		if (ensure(ClientParticipant))
		{
			return ClientParticipant->GetWorld();
		}
	}

	return nullptr;
}

FConversationNodeHandle FConversationContext::GetCurrentNodeHandle() const
{
	if (IsServerContext())
	{
		if (ActiveConversation)
		{
			return ActiveConversation->GetCurrentNodeHandle();
		}
	}
	else
	{
		if (ensure(ClientParticipant))
		{
			return ClientParticipant->GetCurrentNodeHandle();
		}
	}

	return FConversationNodeHandle();
}

FConversationParticipants FConversationContext::GetParticipantsCopy() const
{
	if (IsServerContext())
	{
		if (ActiveConversation)
		{
			return ActiveConversation->GetParticipantsCopy();
		}
	}
	else
	{
		if (ensure(ClientParticipant))
		{
			return ClientParticipant->GetLastMessage().Participants;
		}
	}

	return FConversationParticipants();
}

const FConversationParticipantEntry* FConversationContext::GetParticipant(const FGameplayTag& ParticipantTag) const
{
	if (IsServerContext())
	{
		if (ActiveConversation)
		{
			if (const FConversationParticipantEntry* Participant = ActiveConversation->GetParticipant(ParticipantTag))
			{
				return Participant;
			}
		}
	}
	else
	{
		if (ensure(ClientParticipant))
		{
			return ClientParticipant->GetParticipant(ParticipantTag);
		}
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////

UConversationInstance* UConversationContextHelpers::GetConversationInstance(const FConversationContext& Context)
{
	if (UConversationInstance* Conversation = Context.GetActiveConversation())
	{
		return Conversation;
	}

	LogRuntimeError(LOCTEXT("BadConversationContext", "Bad conversation context"));
	return nullptr;
}

UConversationParticipantComponent* UConversationContextHelpers::GetConversationParticipant(const FConversationContext& Context, FGameplayTag ParticipantTag)
{
	return Context.GetParticipantComponent(ParticipantTag);
}

AActor* UConversationContextHelpers::GetConversationParticipantActor(const FConversationContext& Context, FGameplayTag ParticipantTag)
{
	return Context.GetParticipantActor(ParticipantTag);
}

FConversationNodeHandle UConversationContextHelpers::GetCurrentConversationNodeHandle(const FConversationContext& Context)
{
	if (UConversationInstance* Conversation = GetConversationInstance(Context))
	{
		return Conversation->GetCurrentNodeHandle();
	}

	return FConversationNodeHandle();
}

void UConversationContextHelpers::MakeConversationParticipant(const FConversationContext& Context, AActor* ParticipantActor, FGameplayTag ParticipantTag)
{
#if WITH_SERVER_CODE
	if (ParticipantActor == nullptr)
	{
		LogRuntimeError(LOCTEXT("MakeConversationParticipant_NoActor", "MakeConversationParticipant needs a valid participant actor"));
		return;
	}

	if (!ParticipantTag.IsValid())
	{
		LogRuntimeError(LOCTEXT("MakeConversationParticipant_NoTag", "MakeConversationParticipant needs a valid participant tag"));
		return;
	}

	if (UConversationInstance* Conversation = GetConversationInstance(Context))
	{
		Conversation->ServerAssignParticipant(ParticipantTag, ParticipantActor);
	}
#endif
}

FConversationTaskResult UConversationContextHelpers::AdvanceConversation(const FConversationContext& Context)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::AdvanceConversation();
#else
	return FConversationTaskResult();
#endif
}

FConversationTaskResult UConversationContextHelpers::AdvanceConversationWithChoice(const FConversationContext& Context, const FAdvanceConversationRequest& InPickedChoice)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::AdvanceConversationWithChoice(InPickedChoice);
#else
	return FConversationTaskResult();
#endif
}

FConversationTaskResult UConversationContextHelpers::PauseConversationAndSendClientChoices(const FConversationContext& Context, const FClientConversationMessage& Message)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::PauseConversationAndSendClientChoices(Message);
#else
	return FConversationTaskResult();
#endif
}

FConversationTaskResult UConversationContextHelpers::ReturnToLastClientChoice(const FConversationContext& Context)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::ReturnToLastClientChoice();
#else
	return FConversationTaskResult();
#endif
}

FConversationTaskResult UConversationContextHelpers::ReturnToCurrentClientChoice(const FConversationContext& Context)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::ReturnToCurrentClientChoice();
#else
	return FConversationTaskResult();
#endif
}

FConversationTaskResult UConversationContextHelpers::ReturnToConversationStart(const FConversationContext& Context)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::ReturnToConversationStart();
#else
	return FConversationTaskResult();
#endif
}

FConversationTaskResult UConversationContextHelpers::AbortConversation(const FConversationContext& Context)
{
#if WITH_SERVER_CODE
	return FConversationTaskResult::AbortConversation();
#else
	return FConversationTaskResult();
#endif
}

bool UConversationContextHelpers::CanConversationContinue(const FConversationTaskResult& ConversationTasResult)
{
	return ConversationTasResult.CanConversationContinue();
}

UConversationParticipantComponent* UConversationContextHelpers::FindConversationComponent(AActor* Actor)
{
	if (Actor != nullptr)
	{
		return Actor->FindComponentByClass<UConversationParticipantComponent>();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

