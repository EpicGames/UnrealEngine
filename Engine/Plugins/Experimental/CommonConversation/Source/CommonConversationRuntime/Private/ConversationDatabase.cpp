// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationDatabase.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "Misc/DataValidation.h"
#include "UObject/ObjectSaveContext.h"
#endif
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "UObject/AssetRegistryTagsContext.h"
#if WITH_EDITORONLY_DATA
#include "ConversationNode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationDatabase)

#define LOCTEXT_NAMESPACE "ConversationDatabase"

UConversationDatabase::UConversationDatabase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void UConversationDatabase::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	{
		// Give all nodes a new Guid
		for (UEdGraph* Graph : SourceGraphs)
		{
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				Node->CreateNewGuid();
			}
		}
		
		//@TODO: CONVERSATION: Rebuild/recompile now (can't reach that code in the graph module from here...)
	}
}
#endif

FLinearColor UConversationDatabase::GetDebugParticipantColor(FGameplayTag ParticipantID) const
{
	for (const FCommonDialogueBankParticipant& Speaker : Speakers)
	{
		if (Speaker.ParticipantName == ParticipantID)
		{
			return Speaker.NodeTint;
		}
	}

	return FLinearColor(0.15f, 0.15f, 0.15f);
}

#if WITH_EDITOR
void UConversationDatabase::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
}

EDataValidationResult UConversationDatabase::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult SuperResult = Super::IsDataValid(Context);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FAssetRegistrySerializationOptions SaveOptions;
	AssetRegistry.InitializeSerializationOptions(SaveOptions);

	EDataValidationResult Result = EDataValidationResult::Valid;

	if (SaveOptions.bUseAssetRegistryTagsAllowListInsteadOfDenyList)
	{
		const TSet<FName>* CookedAssetTagsForConversations = SaveOptions.CookFilterlistTagsByClass.Find(UConversationDatabase::StaticClass()->GetClassPathName());
		if (CookedAssetTagsForConversations == nullptr || !CookedAssetTagsForConversations->Contains(GET_MEMBER_NAME_CHECKED(UConversationDatabase, EntryTags)))
		{
			Context.AddError(FText::Format(LOCTEXT("Missing_EntryTags", "Missing from DefaultEngine.ini, {0}"),
				FText::FromString(TEXT("+CookedTagsWhitelist=(Class=ConversationDatabase,Tag=EntryTags)")))
			);

			Result = EDataValidationResult::Invalid;
		}
		if (CookedAssetTagsForConversations == nullptr || !CookedAssetTagsForConversations->Contains(GET_MEMBER_NAME_CHECKED(UConversationDatabase, ExitTags)))
		{
			Context.AddError(FText::Format(LOCTEXT("Missing_ExitTags", "Missing from DefaultEngine.ini, {0}"),
				FText::FromString(TEXT("+CookedTagsWhitelist=(Class=ConversationDatabase,Tag=ExitTags)")))
			);

			Result = EDataValidationResult::Invalid;
		}
		if (CookedAssetTagsForConversations == nullptr || !CookedAssetTagsForConversations->Contains(GET_MEMBER_NAME_CHECKED(UConversationDatabase, InternalNodeIds)))
		{
			Context.AddError(FText::Format(LOCTEXT("Missing_InternalNodeIds", "Missing from DefaultEngine.ini, {0}"),
				FText::FromString(TEXT("+CookedTagsWhitelist=(Class=ConversationDatabase,Tag=InternalNodeIds)")))
			);

			Result = EDataValidationResult::Invalid;
		}
		if (CookedAssetTagsForConversations == nullptr || !CookedAssetTagsForConversations->Contains(GET_MEMBER_NAME_CHECKED(UConversationDatabase, LinkedToNodeIds)))
		{
			Context.AddError(FText::Format(LOCTEXT("Missing_LinkedToNodeIds", "Missing from DefaultEngine.ini, {0}"),
				FText::FromString(TEXT("+CookedTagsWhitelist=(Class=ConversationDatabase,Tag=LinkedToNodeIds)")))
			);

			Result = EDataValidationResult::Invalid;
		}
	}

	EDataValidationResult OutBoundResult = ValidateOutBoundConnections(Context);

	return CombineDataValidationResults(SuperResult, CombineDataValidationResults(Result, OutBoundResult));
}

EDataValidationResult UConversationDatabase::ValidateOutBoundConnections(class FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::Valid;
	for (const TPair<FGuid, TObjectPtr<UConversationNode>>& Node : FullNodeMap)
	{
		if (const UConversationNodeWithLinks* ConversationNode = Cast<UConversationNodeWithLinks>(Node.Value))
		{
			for (const FGuid& OtherNodeGuid : ConversationNode->OutputConnections)
			{
				if (const UConversationNodeWithLinks* OtherConversationNode = Cast<UConversationNodeWithLinks>(FullNodeMap[OtherNodeGuid]))
				{
					FText ErrorMessage;
					if (!ConversationNode->IsOutBoundConnectionAllowed(OtherConversationNode, ErrorMessage))
					{
						Context.AddError(FText::Format(LOCTEXT("OutBoundConnectionError", "({0}) -> ({1}) : {2}"),
							ConversationNode->GetDisplayNameText(), OtherConversationNode->GetDisplayNameText(), ErrorMessage));

						Result = EDataValidationResult::Invalid;
					}
				}
			}
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
FGuid UConversationDatabase::GetGuidFromNode(const UConversationNode* NodeToFind) const
{
	for (const TPair<FGuid, TObjectPtr<UConversationNode>>& Node : FullNodeMap)
	{
		if (Node.Value == NodeToFind)
		{
			return Node.Key;
		}
	}
	return FGuid();
}

TObjectPtr<UEdGraphNode> UConversationDatabase::GetSourceGraphNodeFromGuid(FGuid NodeToFind) const
{
	for (const TObjectPtr<UEdGraph>& Graph : SourceGraphs)
	{
		for (TObjectPtr<UEdGraphNode>& Node : Graph->Nodes)
		{
			if (Node->NodeGuid == NodeToFind)
			{
				return Node;
			}
		}
	}
	return TObjectPtr<UEdGraphNode>();
}
#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
