// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationEntryPointNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationEntryPointNode)

FString UConversationEntryPointNode::GetIdentifier()
{
	return FString();
}