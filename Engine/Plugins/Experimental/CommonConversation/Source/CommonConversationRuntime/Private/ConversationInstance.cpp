// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationInstance.h"
#include "ConversationRegistry.h"
#include "CommonConversationRuntimeLogging.h"
#include "ConversationContext.h"
#include "ConversationTaskNode.h"
#include "ConversationChoiceNode.h"
#include "Engine/World.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationInstance)

namespace ConversationInstanceCVars
{
	static bool bShouldAbortConversationOnInvalidChoice = false;
	static FAutoConsoleVariableRef CVarShouldAbortConversationOnInvalidChoice(
		TEXT("Conversation.Instance.AbortConversationOnInvalidChoice"), bShouldAbortConversationOnInvalidChoice,
		TEXT("About the conversation when an invalid choice is chosen."),
		ECVF_Default);
}

namespace UE::ConversationCommon::Private
{
	static EConversationNodeBranchOrder FindBranchOrderFromChoice(const FConversationContext& Context, const FConversationChoiceReference& PreviousNode)
	{
		if (const UConversationTaskNode* ConversationNode = Cast<UConversationTaskNode>(PreviousNode.NodeReference.TryToResolve(Context)))
		{
			return ConversationNode->BranchOrder;
		}

		return EConversationNodeBranchOrder::Random;
	}
}

//@TODO: CONVERSATION: Assert or otherwise guard all the Server* functions to only execute on the authority

//////////////////////////////////////////////////////////////////////

UConversationInstance::UConversationInstance()
{
}

UWorld* UConversationInstance::GetWorld() const
{
	return GetTypedOuter<UWorld>();
}

#if WITH_SERVER_CODE

void UConversationInstance::ServerRemoveParticipant(const FGameplayTag& ParticipantID, const FConversationParticipants& PreservedParticipants)
{
	for (auto It = Participants.List.CreateIterator(); It; ++It)
	{
		if (It->ParticipantID == ParticipantID)
		{
			if (UConversationParticipantComponent* OldParticipant = It->GetParticipantComponent())
			{
				if (bConversationStarted)
				{
					OldParticipant->ServerNotifyConversationEnded(this, PreservedParticipants);
				}
			}
			It.RemoveCurrent();
			break;
		}
	}
}

void UConversationInstance::ServerAssignParticipant(const FGameplayTag& ParticipantID, AActor* ParticipantActor)
{
	if (!ParticipantID.IsValid() || (ParticipantActor == nullptr))
	{
		UE_LOGF(LogCommonConversationRuntime, Error, "AConversationInstance::ServerAssignParticipant(ID=%ls, Actor=%ls) passed bad arguments",
			*ParticipantID.ToString(), *GetPathNameSafe(ParticipantActor));
		return;
	}

	ServerRemoveParticipant(ParticipantID, Participants);

	FConversationParticipantEntry NewEntry;
	NewEntry.ParticipantID = ParticipantID;
	NewEntry.Actor = ParticipantActor;
	Participants.List.Add(NewEntry);

	if (bConversationStarted)
	{
		if (UConversationParticipantComponent* ParticipantComponent = NewEntry.GetParticipantComponent())
		{
			ParticipantComponent->ServerNotifyConversationStarted(this, ParticipantID);
		}
	}

	UE_LOGF(LogCommonConversationRuntime, Verbose, "Conversation %ls assigned participant ID=%ls to Actor=%ls",
		*GetName(), *ParticipantID.ToString(), *GetPathNameSafe(ParticipantActor));
}

void UConversationInstance::ServerStartConversation(const FGameplayTag& EntryPoint, const UConversationDatabase* Graph, const FString& EntryPointIdentifier)
{
	UE_LOGF(LogCommonConversationRuntime, Verbose, "Conversation %ls starting at %ls with %d participants",
		*GetName(), *EntryPoint.ToString(), Participants.List.Num());

	ResetConversationProgress();
	StartingEntryGameplayTag = EntryPoint;
	ActiveConversationGraph = Graph;

	UConversationRegistry* ConversationRegistry = UConversationRegistry::GetFromWorld(GetWorld());
	
	TArray<FGuid> PotentialStartingPoints = ConversationRegistry->GetOutputLinkGUIDs(Graph, EntryPoint, EntryPointIdentifier);
	if (PotentialStartingPoints.Num() == 0)
	{
		UE_LOGF(LogCommonConversationRuntime, Warning, "Entry point %ls did not exist or had no destination entries; conversation aborted", *EntryPoint.ToString());
		ServerAbortConversation();
		return;
	}
	else
	{
		TArray<FGuid> LegalStartingPoints = DetermineBranches(PotentialStartingPoints, EConversationRequirementResult::FailedButVisible);

		if (LegalStartingPoints.Num() == 0)
		{
			UE_LOGF(LogCommonConversationRuntime, Verbose, "All branches from entry point %ls are disabled, conversation aborted", *EntryPoint.ToString());
			ServerAbortConversation();
			return;
		}
		else
		{
			const int32 StartingIndex = ChooseBranchIndex(EConversationNodeBranchOrder::Random, LegalStartingPoints.Num());
			
			FConversationBranchPoint StartingPoint;
			StartingPoint.ClientChoice.ChoiceReference = FConversationChoiceReference(LegalStartingPoints[StartingIndex]);
			
			StartingBranchPoint = StartingPoint;
			CurrentBranchPoint = StartingPoint;
			
			UE_LOGF(LogCommonConversationRuntime, Verbose, "Chosing branch index %d to %ls (of %d legal branches) from entry point %ls",
				StartingIndex, *CurrentBranchPoint.ClientChoice.ChoiceReference.ToString(), LegalStartingPoints.Num(), *EntryPoint.ToString());
		}
	}

	for (const FConversationParticipantEntry& ParticipantEntry : Participants.List)
	{
		if (UConversationParticipantComponent* ParticipantComponent = ParticipantEntry.GetParticipantComponent())
		{
			ParticipantComponent->ServerNotifyConversationStarted(this, ParticipantEntry.ParticipantID);
		}
	}

	bConversationStarted = true;
	OnAllParticipantsNotifiedOfStart.Broadcast(this);

	TryStartingConversation();
}

bool UConversationInstance::AreAllParticipantsReadyToConverse() const
{
	bool bEveryoneReady = true;
	for (const FConversationParticipantEntry& ParticipantEntry : Participants.List)
	{
		if (UConversationParticipantComponent* ParticipantComponent = ParticipantEntry.GetParticipantComponent())
		{
			if (!ParticipantComponent->ServerIsReadyToConverse())
			{
				ParticipantComponent->ServerGetReadyToConverse();
				bEveryoneReady = false;
			}
		}
	}

	return bEveryoneReady;
}

void UConversationInstance::TryStartingConversation()
{
	// If the conversation was aborted, nevermind.
	if (!bConversationStarted)
	{
		return;
	}

	if (!AreAllParticipantsReadyToConverse())
	{
		for (const FConversationParticipantEntry& ParticipantEntry : Participants.List)
		{
			if (UConversationParticipantComponent* ParticipantComponent = ParticipantEntry.GetParticipantComponent())
			{
				ParticipantComponent->OnParticipantReadyToConverseEvent.RemoveAll(this);
				ParticipantComponent->OnParticipantReadyToConverseEvent.AddWeakLambda(this, [this](UConversationParticipantComponent* ReadyComponent) {
					TryStartingConversation();
				});
			}
		}
	}
	else
	{
		// Flush any still listening handlers.
		for (const FConversationParticipantEntry& ParticipantEntry : Participants.List)
		{
			if (UConversationParticipantComponent* ParticipantComponent = ParticipantEntry.GetParticipantComponent())
			{
				ParticipantComponent->OnParticipantReadyToConverseEvent.RemoveAll(this);
			}
		}

		ConversationRNG.Initialize(NAME_None);

		OnStarted();

		OnCurrentConversationNodeModified();
	}
}

void UConversationInstance::ServerAdvanceConversation(const FAdvanceConversationRequest& InChoicePicked)
{
	if (bConversationStarted && GetCurrentChoiceReference().IsValid())
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "ServerAdvanceConversation is determining destinations from %ls", *GetCurrentChoiceReference().ToString());

		TArray<FConversationBranchPoint> CandidateDestinations;

		const FConversationContext ServerContext = FConversationContext::CreateServerContext(this, nullptr);
		const UConversationChoiceNode* ChoiceNodePicked = nullptr;

		if (InChoicePicked.Choice != FConversationChoiceReference::Empty)
		{
			if (const FConversationBranchPoint* BranchPoint = FindBranchPointFromClientChoice(InChoicePicked.Choice))
			{
				UE_LOGF(LogCommonConversationRuntime, Verbose, "User picked option %ls, going to try that", *BranchPoint->ClientChoice.ChoiceReference.ToString());
				CandidateDestinations = { *BranchPoint };

				if (const UConversationTaskNode* TaskNode = BranchPoint->ClientChoice.TryToResolveChoiceNode<UConversationTaskNode>(ServerContext))
				{
					for (const UConversationNode* SubNode : TaskNode->SubNodes)
					{
						if (const UConversationChoiceNode* ChoiceNode = Cast<UConversationChoiceNode>(SubNode))
						{
							ChoiceNodePicked = ChoiceNode;
							ChoiceNode->NotifyChoicePickedByUser(ServerContext, BranchPoint->ClientChoice);
							break;
						}
					}
				}
			}
			else
			{
				OnInvalidBranchChoice(InChoicePicked);
				return;
			}
		}
		else
		{
			if (CurrentBranchPoints.Num() == 0 && ScopeStack.Num() > 0)
			{
				ModifyCurrentConversationNode(ScopeStack.Top());
				return;
			}

			for (const FConversationBranchPoint& BranchPoint : CurrentBranchPoints)
			{
				if (BranchPoint.ClientChoice.IsChoiceAvailable())
				{
					CandidateDestinations.Add(BranchPoint);
				}
			}
		}

		// Double check the choices are still valid, things may have changed since the user picked the choices.
		TArray<FConversationBranchPoint> ValidDestinations;
		{
			FConversationContext Context = FConversationContext::CreateServerContext(this, nullptr);
			for (const FConversationBranchPoint& BranchPoint : CandidateDestinations)
			{
				if (const UConversationTaskNode* TaskNode = BranchPoint.ClientChoice.TryToResolveChoiceNode<UConversationTaskNode>(Context))
				{	
					EConversationRequirementResult Result = TaskNode->bIgnoreRequirementsWhileAdvancingConversations ? EConversationRequirementResult::Passed : TaskNode->CheckRequirements(Context);
					if (Result == EConversationRequirementResult::Passed)
					{
						ValidDestinations.Add(BranchPoint);
					}
				}
				else
				{
					ValidDestinations.Add(BranchPoint);
				}
			}
		}

		// Allow derived conversation instances a chance to respond to a choice being picked
		if (ChoiceNodePicked)
		{ 
			OnChoiceNodePickedByUser(ServerContext, ChoiceNodePicked, ValidDestinations);
		}

		if (ValidDestinations.Num() == 0)
		{
			UE_LOGF(LogCommonConversationRuntime, Verbose, "No available destinations from %ls, ending the conversation", *GetCurrentChoiceReference().ToString());
			ServerAbortConversation();
			return;
		}
		else
		{
			const FConversationChoiceReference& PreviousNode = GetCurrentChoiceReference();
			const EConversationNodeBranchOrder BranchOrder = UE::ConversationCommon::Private::FindBranchOrderFromChoice(ServerContext, PreviousNode);
			const int32 StartingIndex = ChooseBranchIndex(BranchOrder, ValidDestinations.Num());
			const FConversationBranchPoint& TargetChoice = ValidDestinations[StartingIndex];
			
			UE_LOGF(LogCommonConversationRuntime, Verbose, "Chosing destination index %d to %ls (of %d legal branches) from %ls using order %ls",
				StartingIndex, *TargetChoice.ClientChoice.ChoiceReference.ToString(), ValidDestinations.Num(), *PreviousNode.ToString(), *UEnum::GetValueAsString(BranchOrder));

			ModifyCurrentConversationNode(TargetChoice);
		}
	}
	else
	{
		UE_LOGF(LogCommonConversationRuntime, Error, "ServerAdvanceConversation called when the conversation is not active");
	}
}

void UConversationInstance::OnInvalidBranchChoice(const FAdvanceConversationRequest& InChoicePicked)
{
	if (ConversationInstanceCVars::bShouldAbortConversationOnInvalidChoice)
	{
		UE_LOGF(LogCommonConversationRuntime, Error, "User picked option %ls but it's not a legal output, aborting", *InChoicePicked.ToString());
		ServerAbortConversation();
	}
	else
	{
		UE_LOGF(LogCommonConversationRuntime, Warning, "User picked option %ls but it's not a legal output, ignoring", *InChoicePicked.ToString());

		// Forces the client to refresh current choices in case it is not aligned with the server
		FConversationContext Context = FConversationContext::CreateServerContext(this, nullptr);
		const bool bForceClientRefresh = true;

		for (const FConversationParticipantEntry& ConversationParticipantEntry : GetParticipantListCopy())
		{
			if (UConversationParticipantComponent* ParticipantComponent = ConversationParticipantEntry.GetParticipantComponent())
			{
				ParticipantComponent->SendClientUpdatedChoices(Context, bForceClientRefresh);
			}
		}
	}
}

void UConversationInstance::ServerAbortConversation()
{
	if (bConversationStarted)
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "Conversation aborted or finished");

		OnEnded();

		const FConversationParticipants ParticipantsCopy = Participants;
		for (const FConversationParticipantEntry& ParticipantEntry : ParticipantsCopy.List)
		{
			ServerRemoveParticipant(ParticipantEntry.ParticipantID, ParticipantsCopy);
		}
		check(Participants.List.Num() == 0);
	}

	ResetConversationProgress();

	bConversationStarted = false;
}

void UConversationInstance::PauseConversationAndSendClientChoices(const FConversationContext& Context, const FClientConversationMessage& ClientMessage)
{
	FClientConversationMessagePayload LastMessage = FClientConversationMessagePayload();
	LastMessage.Message = ClientMessage;
	LastMessage.Options = CurrentUserChoices;
	LastMessage.CurrentNode = Context.GetCurrentNodeHandle();
	LastMessage.Participants = GetParticipantsCopy();

	ClientBranchPoints.Add({ CurrentBranchPoint, ScopeStack });

	for (const FConversationParticipantEntry& KVP : LastMessage.Participants.List)
	{
		if (UConversationParticipantComponent* ParticipantComponent = KVP.GetParticipantComponent())
		{
			ParticipantComponent->SendClientConversationMessage(Context, LastMessage);
		}
	}
}

void UConversationInstance::ReturnToLastClientChoice(const FConversationContext& Context)
{
	if (ClientBranchPoints.Num() > 1)
	{
		ClientBranchPoints.Pop();

		const FCheckpoint& Checkpoint = ClientBranchPoints[ClientBranchPoints.Num() - 1];
		ScopeStack = Checkpoint.ScopeStack;
		ModifyCurrentConversationNode(Checkpoint.ClientBranchPoint);
	}
}

void UConversationInstance::ReturnToCurrentClientChoice(const FConversationContext& Context)
{
	if (ClientBranchPoints.Num() > 0)
	{
		const FCheckpoint Checkpoint = ClientBranchPoints[ClientBranchPoints.Num() - 1];

		// Pop after get the last checkpoint since we're about to repeat and push the same one onto the stack.
		ClientBranchPoints.Pop();

		ScopeStack = Checkpoint.ScopeStack;
		ModifyCurrentConversationNode(Checkpoint.ClientBranchPoint);
	}
}

void UConversationInstance::ReturnToStart(const FConversationContext& Context)
{
	FGameplayTag EntryStartPointGameplayTagCache = StartingEntryGameplayTag;
	FConversationBranchPoint StartingBranchPointCache = StartingBranchPoint;

	ResetConversationProgress();

	StartingEntryGameplayTag = EntryStartPointGameplayTagCache;
	StartingBranchPoint = StartingBranchPointCache;

	ModifyCurrentConversationNode(StartingBranchPoint);
}

void UConversationInstance::ModifyCurrentConversationNode(const FConversationChoiceReference& NewChoice)
{
	FConversationBranchPoint BranchPoint;
	BranchPoint.ClientChoice.ChoiceReference = NewChoice;

	ModifyCurrentConversationNode(BranchPoint);
}

void UConversationInstance::ModifyCurrentConversationNode(const FConversationBranchPoint& NewBranchPoint)
{
	UE_LOGF(LogCommonConversationRuntime, Verbose, "Modying Current Node From %ls To %ls",
		*NewBranchPoint.ClientChoice.ChoiceReference.ToString(), *GetCurrentChoiceReference().ToString());

	CurrentBranchPoint = NewBranchPoint;
	ScopeStack.Append(NewBranchPoint.ReturnScopeStack);

	OnCurrentConversationNodeModified();
}

void UConversationInstance::ServerRefreshConversationChoices()
{
	FConversationContext Context = FConversationContext::CreateServerContext(this, nullptr);

	// Update the next choices now that we've executed the task
	UpdateNextChoices(Context);

	for (const FConversationParticipantEntry& KVP : GetParticipantListCopy())
	{
		if (UConversationParticipantComponent* ParticipantComponent = KVP.GetParticipantComponent())
		{
			ParticipantComponent->SendClientUpdatedChoices(Context);
		}
	}
}

void UConversationInstance::ServerRefreshTaskChoiceData(const FConversationNodeHandle& Handle)
{
	FConversationContext Context = FConversationContext::CreateServerContext(this, nullptr);

	// Technically we only need to do a gather for a single choice here from the current active subset, but gathering all for now (only choice data relevant to Handle will actually be sent)
	UpdateNextChoices(Context);
	
	for (const FConversationParticipantEntry& KVP : GetParticipantListCopy())
	{
		if (UConversationParticipantComponent* ParticipantComponent = KVP.GetParticipantComponent())
		{
			ParticipantComponent->SendClientRefreshedTaskChoiceData(Handle, Context);
		}
	}
}

void UConversationInstance::ServerRefreshCurrentConversationNode()
{
	ProcessCurrentConversationNode();
}

#endif // #if WITH_SERVER_CODE

void UConversationInstance::ResetConversationProgress()
{
	StartingEntryGameplayTag = FGameplayTag();
	StartingBranchPoint = FConversationBranchPoint();
	CurrentBranchPoint = FConversationBranchPoint();
	CurrentBranchPoints.Reset();
	ClientBranchPoints.Reset();
	CurrentUserChoices.Reset();
}

void UConversationInstance::UpdateNextChoices(const FConversationContext& Context)
{
	TArray<FConversationBranchPoint> AllChoices;

	if (const UConversationTaskNode* TaskNode = Cast<UConversationTaskNode>(GetCurrentChoiceReference().NodeReference.TryToResolve(Context)))
	{
		FConversationContext ChoiceContext = Context.CreateChildContext(TaskNode);
		FConversationBranchPointBuilder BranchBuilder;

		TaskNode->GenerateNextChoices(BranchBuilder, ChoiceContext);

		AllChoices = BranchBuilder.GetBranches();
	}

	SetNextChoices(AllChoices);
}

void UConversationInstance::SetNextChoices(const TArray<FConversationBranchPoint>& InAllChoices)
{
	CurrentUserChoices.Reset();
	CurrentBranchPoints = InAllChoices;

	for (const FConversationBranchPoint& UserBranchPoint : CurrentBranchPoints)
	{
		if (UserBranchPoint.ClientChoice.ChoiceType != EConversationChoiceType::ServerOnly)
		{
			CurrentUserChoices.Add(UserBranchPoint.ClientChoice);
		}
	}

	if (CurrentBranchPoints.Num() > 0 || ScopeStack.Num() > 0)
	{
		if (CurrentUserChoices.Num() == 0)
		{
			FClientConversationOptionEntry DefaultChoice;
			DefaultChoice.ChoiceReference = FConversationChoiceReference::Empty;
			DefaultChoice.ChoiceText = NSLOCTEXT("ConversationInstance", "ConversationInstance_DefaultText", "Continue");
			DefaultChoice.ChoiceType = EConversationChoiceType::UserChoiceAvailable;
			CurrentUserChoices.Add(DefaultChoice);
		}
	}
}

const FConversationBranchPoint* UConversationInstance::FindBranchPointFromClientChoice(const FConversationChoiceReference& InChoice) const
{
	return CurrentBranchPoints.FindByPredicate([&InChoice](const FConversationBranchPoint& InBranchPoint) {
		if (InBranchPoint.ClientChoice.ChoiceType != EConversationChoiceType::ServerOnly)
		{
			return InBranchPoint.ClientChoice == InChoice;
		}
		return false;
	});
}

#if WITH_SERVER_CODE

TArray<FGuid> UConversationInstance::DetermineBranches(const TArray<FGuid>& SourceList, EConversationRequirementResult MaximumRequirementResult)
{
	FConversationContext Context = FConversationContext::CreateServerContext(this, nullptr);

	TArray<FGuid> EnabledPaths;
	for (const FGuid& TestGUID : SourceList)
	{
		UConversationNode* TestNode = Context.GetConversationRegistry().GetRuntimeNodeFromGUID(TestGUID, ActiveConversationGraph.Get());
		if (UConversationTaskNode* TaskNode = Cast<UConversationTaskNode>(TestNode))
		{
			const EConversationRequirementResult RequirementResult = TaskNode->CheckRequirements(Context);

			if ((int64)RequirementResult <= (int64)MaximumRequirementResult)
			{
				UE_LOGF(LogCommonConversationRuntime, Verbose, "\t%ls is legal", *TestGUID.ToString());
				EnabledPaths.Add(TestGUID);
			}
		}
	}

	UE_LOGF(LogCommonConversationRuntime, Verbose, "\t%d paths out of %d are legal", EnabledPaths.Num(), SourceList.Num());
	return EnabledPaths;
}

void UConversationInstance::OnCurrentConversationNodeModified()
{
	ProcessCurrentConversationNode();
}

void UConversationInstance::ProcessCurrentConversationNode()
{
	check(GetCurrentChoiceReference().IsValid());

	FConversationContext AnonContext = FConversationContext::CreateServerContext(this, nullptr);
	const UConversationNode* CurrentNode = GetCurrentChoiceReference().NodeReference.TryToResolve(AnonContext);
	if (const UConversationTaskNode* TaskNode = Cast<UConversationTaskNode>(CurrentNode))
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "Executing task node %ls", *GetCurrentChoiceReference().ToString());

		FConversationContext Context = AnonContext.CreateChildContext(TaskNode);

		const FConversationTaskResult TaskResult = TaskNode->ExecuteTaskNodeWithSideEffects(Context);

		if (ScopeStack.Num() > 0 && ScopeStack.Top() == TaskNode->GetNodeGuid())
		{
			// Now that we've finally executed the Subgraph / scope modifying node, we can pop it from
			// the scope stack.
			ScopeStack.Pop();
		}

		// Update the next choices now that we've executed the task
		UpdateNextChoices(Context);

		if (TaskResult.GetType() == EConversationTaskResultType::AbortConversation)
		{
			ServerAbortConversation();
		}
		else if (TaskResult.GetType() == EConversationTaskResultType::AdvanceConversation)
		{
			ServerAdvanceConversation(FAdvanceConversationRequest::Any);
		}
		else if (TaskResult.GetType() == EConversationTaskResultType::AdvanceConversationWithChoice)
		{
			//@TODO: CONVERSATION: We are only using the Choice here part of the request, but we need to complete
			// support UserParameters, just so we don't have unexpected differences in the system.
			ModifyCurrentConversationNode(TaskResult.GetChoice().Choice);
		}
		else if (TaskResult.GetType() == EConversationTaskResultType::PauseConversationAndSendClientChoices)
		{
			PauseConversationAndSendClientChoices(Context, TaskResult.GetMessage());
		}
		else if (TaskResult.GetType() == EConversationTaskResultType::ReturnToLastClientChoice)
		{
			ReturnToLastClientChoice(Context);
		}
		else if (TaskResult.GetType() == EConversationTaskResultType::ReturnToCurrentClientChoice)
		{
			ReturnToCurrentClientChoice(Context);
		}
		else if (TaskResult.GetType() == EConversationTaskResultType::ReturnToConversationStart)
		{
			ReturnToStart(Context);
		}
		else
		{
			ensureMsgf(false, TEXT("Invalid ResultType executing task node %s"), *GetCurrentChoiceReference().ToString());
		}
	}
	else
	{
		UE_LOGF(LogCommonConversationRuntime, Error, "Ended up with no task node with ID %ls, aborting conversation", *GetCurrentChoiceReference().ToString());

		ServerAbortConversation();
	}
}

int32 UConversationInstance::ChooseBranchIndex(EConversationNodeBranchOrder BranchOrder, int32 NumBranchIndices)
{
	switch (BranchOrder)
	{
	case EConversationNodeBranchOrder::LeftToRight:
		return 0;
	case EConversationNodeBranchOrder::RightToLeft:
		return FMath::Max(NumBranchIndices - 1, 0);
	case EConversationNodeBranchOrder::Random:
	default:
		return ConversationRNG.RandRange(0, NumBranchIndices - 1);
	}
}

#endif //WITH_SERVER_CODE
