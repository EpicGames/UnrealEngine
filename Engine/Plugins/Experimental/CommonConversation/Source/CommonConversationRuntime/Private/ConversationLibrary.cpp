// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationLibrary.h"
#include "ConversationContext.h"
#include "ConversationSettings.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationLibrary)

#define LOCTEXT_NAMESPACE "ConversationLibrary"

static UConversationInstance* StartConversationShared(const FGameplayTag& ConversationEntryTag, AActor* Instigator, const FGameplayTag& InstigatorTag,
	AActor* Target, const FGameplayTag& TargetTag, const TSubclassOf<UConversationInstance> ConversationInstanceClass, const UConversationDatabase* Graph, const FString& EntryPointIdentifier)
{
#if WITH_SERVER_CODE
	if (Instigator == nullptr || Target == nullptr || GEngine == nullptr)
	{
		return nullptr;
	}

	if (UWorld* World = GEngine->GetWorldFromContextObject(Instigator, EGetWorldErrorMode::LogAndReturnNull))
	{
		UClass* InstanceClass = ConversationInstanceClass;
		if (!InstanceClass)
		{
			InstanceClass = GetDefault<UConversationSettings>()->GetConversationInstanceClass();
			if (InstanceClass == nullptr)
			{
				InstanceClass = UConversationInstance::StaticClass();
			}
		}
		UConversationInstance* ConversationInstance = NewObject<UConversationInstance>(World, InstanceClass);
		if (ensure(ConversationInstance))
		{
			FConversationContext Context = FConversationContext::CreateServerContext(ConversationInstance, nullptr);

			UConversationContextHelpers::MakeConversationParticipant(Context, Target, TargetTag);
			UConversationContextHelpers::MakeConversationParticipant(Context, Instigator, InstigatorTag);

			ConversationInstance->ServerStartConversation(ConversationEntryTag, Graph, EntryPointIdentifier);
		}

		return ConversationInstance;
	}
#endif

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// UConversationLibrary

UConversationLibrary::UConversationLibrary()
{
}

UConversationInstance* UConversationLibrary::StartConversation(const FGameplayTag& ConversationEntryTag, AActor* Instigator,
	const FGameplayTag& InstigatorTag, AActor* Target, const FGameplayTag& TargetTag, const TSubclassOf<UConversationInstance> ConversationInstanceClass)
{
	return StartConversationShared(ConversationEntryTag, Instigator, InstigatorTag, Target, TargetTag, ConversationInstanceClass, nullptr, FString());
}

UConversationInstance* UConversationLibrary::StartConversationFromGraph(const FGameplayTag& ConversationEntryTag, AActor* Instigator, const FGameplayTag& InstigatorTag,
	AActor* Target, const FGameplayTag& TargetTag, const UConversationDatabase* Graph, const FString& EntryPointIdentifier)
{
	return (Graph) ? StartConversationShared(ConversationEntryTag, Instigator, InstigatorTag, Target, TargetTag, nullptr, Graph, EntryPointIdentifier) : nullptr;
}

#undef LOCTEXT_NAMESPACE

