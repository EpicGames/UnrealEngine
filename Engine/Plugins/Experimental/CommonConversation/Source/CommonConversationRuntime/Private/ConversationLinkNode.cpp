// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationLinkNode.h"
#include "ConversationContext.h"
#include "ConversationRegistry.h"
#include "ConversationInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationLinkNode)

#define LOCTEXT_NAMESPACE "ConversationLinkNode"

UConversationLinkNode::UConversationLinkNode()
{
#if WITH_EDITORONLY_DATA
	bHasDynamicChoices = true;
#endif
}

FConversationTaskResult UConversationLinkNode::ExecuteTaskNode_Implementation(const FConversationContext& Context) const
{
	return FConversationTaskResult::AdvanceConversation();
}

void UConversationLinkNode::GatherChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const
{
#if WITH_SERVER_CODE
	TArray<FGuid> PotentialStartingPoints = Context.GetConversationRegistry().GetOutputLinkGUIDs(RemoteEntryTag);
	
	if (PotentialStartingPoints.Num() > 0)
	{
		TArray<FGuid> LegalStartingPoints = Context.GetActiveConversation()->DetermineBranches(PotentialStartingPoints, EConversationRequirementResult::FailedButVisible);
		FConversationContext ReturnScopeContext = Context.CreateReturnScopeContext(GetNodeGuid());
		UConversationTaskNode::GenerateChoicesForDestinations(BranchBuilder, ReturnScopeContext, LegalStartingPoints);
	}
#endif
}

#undef LOCTEXT_NAMESPACE
