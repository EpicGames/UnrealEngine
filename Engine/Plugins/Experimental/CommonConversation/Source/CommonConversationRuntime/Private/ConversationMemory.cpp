// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationMemory.h"
#include "ConversationTaskNode.h"

FConversationMemory::FConversationTaskMemoryKey::FConversationTaskMemoryKey(const UConversationTaskNode& SourceTaskInstance, const UScriptStruct* TaskMemoryStructClass)
	: SourceTaskInstanceKey(&SourceTaskInstance)
	, TaskMemoryStructClassKey(TaskMemoryStructClass)
	, KeyHash(0)
{
	KeyHash = HashCombine(KeyHash, GetTypeHash(SourceTaskInstanceKey));
	KeyHash = HashCombine(KeyHash, GetTypeHash(TaskMemoryStructClassKey));
}

const UScriptStruct* FConversationMemory::FConversationTaskMemoryKey::GetTaskMemoryStruct() const
{
	const UObject* TaskMemoryStructClass = TaskMemoryStructClassKey.ResolveObjectPtr();
	return Cast<UScriptStruct>(TaskMemoryStructClass);
}

FConversationMemory::~FConversationMemory()
{
	for (auto& TaskMemoryEntry : TaskMemory)
	{
		if (const UScriptStruct* TaskMemoryType = TaskMemoryEntry.Key.GetTaskMemoryStruct())
		{
			TaskMemoryType->DestroyStruct(TaskMemoryEntry.Value);
		}
	}
}

void* FConversationMemory::GetTaskMemoryOfType(const UConversationTaskNode& Task, const UScriptStruct* TaskMemoryStructType)
{
	FConversationTaskMemoryKey TaskMemoryKey(Task, TaskMemoryStructType);
	void** ExistingMemoryPtr = TaskMemory.Find(TaskMemoryKey);
	if (ExistingMemoryPtr == nullptr)
	{
		void* NewTaskMemory = TaskMemoryAllocator.Allocate(TaskMemoryStructType->GetStructureSize());
		TaskMemoryStructType->InitializeStruct(NewTaskMemory);

		TaskMemory.Add(TaskMemoryKey, NewTaskMemory);
		return NewTaskMemory;
	}

	return *ExistingMemoryPtr;
}
