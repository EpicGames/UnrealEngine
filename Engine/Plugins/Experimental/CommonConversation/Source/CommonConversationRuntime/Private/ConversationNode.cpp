// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationNode.h"

#include "Engine/World.h"
#include "UObject/Package.h"
#include "BlueprintNodeHelpers.h"
#include "ConversationDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationNode)

#define LOCTEXT_NAMESPACE "ConversationGraph"

//////////////////////////////////////////////////////////////////////
// UConversationNode

UConversationNode::UConversationNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ParentNode = nullptr;

#if WITH_EDITORONLY_DATA
	bShowPropertyDetails = true;
#endif
}

UWorld* UConversationNode::GetWorld() const
{
	if (GetOuter() == nullptr)
	{
		return nullptr;
	}

	if (EvalWorldContextObj != nullptr)
	{
		return EvalWorldContextObj->GetWorld();
	}

	// Special case for behavior tree nodes in the editor
	if (Cast<UPackage>(GetOuter()) != nullptr)
	{
		// GetOuter should return a UPackage and its Outer is a UWorld
		return Cast<UWorld>(GetOuter()->GetOuter());
	}

	// In all other cases...
	return GetOuter()->GetWorld();
}

void UConversationNode::InitializeNode(UConversationNode* InParentNode)
{
	ParentNode = InParentNode;
}

void UConversationNode::InitializeFromAsset(UConversationDatabase& Asset)
{
//	BankAsset = &Asset;
}

FText UConversationNode::GetDisplayNameText() const
{
	return NodeName.Len() ? 
		FText::FromString(NodeName)
	:
#if WITH_EDITOR
		GetClass()->GetDisplayNameText();
#else
		FText::FromName(GetClass()->GetFName());
#endif
}

FText UConversationNode::GetRuntimeDescription(const UCommonDialogueConversation& OwnerComp, EConversationNodeDescriptionVerbosity Verbosity) const
{
	TArray<FString> RuntimeValues;
	DescribeRuntimeValues(OwnerComp, Verbosity, RuntimeValues);

	return FText::Format(LOCTEXT("NodeRuntimeDescription", "{0} [{1}]{2}"),
		GetDisplayNameText(),
		GetStaticDescription(),
		FText::FromString(FString::Join(RuntimeValues, TEXT(", ")))
	);
}

FText UConversationNode::GetStaticDescription() const
{
#if WITH_EDITOR
	UConversationNode* CDO = GetClass()->GetDefaultObject<UConversationNode>();
	if (ShowPropertyDetails() && CDO)
	{
		const UClass* StopAtClass = UConversationNode::StaticClass();
		const FString PropertyDesc = BlueprintNodeHelpers::CollectPropertyDescription(this, StopAtClass, [this](FProperty* InTestProperty) 
		{
			return ShouldHideProperty(InTestProperty);
		});

		return FText::FromString(PropertyDesc);
	}
#endif // WITH_EDITOR

	return GetDisplayNameText();
}

#if WITH_EDITOR
bool UConversationNode::ShouldHideProperty(FProperty* InTestProperty) const
{
	const static FName HideInConversationMeta{ TEXT("HideInConversationNode") };
	if (InTestProperty->HasMetaData(HideInConversationMeta))
	{
		return true;
	}
	// base property for single structs
	else if (FStructProperty* StructTestProperty = CastField<FStructProperty>(InTestProperty))
	{
		return StructTestProperty->Struct->HasMetaData(HideInConversationMeta);
	}
	// base property for strong/soft object pointers
	else if (FObjectPropertyBase* ObjectTestProperty = CastField<FObjectPropertyBase>(InTestProperty))
	{
		return ObjectTestProperty->PropertyClass->HasMetaData(HideInConversationMeta);
	}
	// container support for this would be cleanest if we pulled everything but the first if into a helper, eg ShouldHideProp(ArrayProperty->Element)
	return false;
}
#endif // #if WITH_EDITOR

void UConversationNode::DescribeRuntimeValues(const UCommonDialogueConversation& OwnerComp, EConversationNodeDescriptionVerbosity Verbosity, TArray<FString>& Values) const
{
	// nothing stored in memory for base class
}

#if WITH_EDITOR
FName UConversationNode::GetNodeIconName() const
{
	return NAME_None;
}
#endif

FLinearColor UConversationNode::GetDebugParticipantColor(FGameplayTag ParticipantID) const
{
	UConversationDatabase* Bank = CastChecked<UConversationDatabase>(GetOuter());
	return Bank->GetDebugParticipantColor(ParticipantID);
}

#undef LOCTEXT_NAMESPACE
