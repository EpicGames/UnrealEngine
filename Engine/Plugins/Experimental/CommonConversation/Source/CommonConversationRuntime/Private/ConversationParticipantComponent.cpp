// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationParticipantComponent.h"
#include "ConversationContext.h"
#include "ConversationInstance.h"
#include "Net/UnrealNetwork.h"
#include "ConversationTaskNode.h"
#include "CommonConversationRuntimeLogging.h"
#include "Net/Core/PushModel/PushModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationParticipantComponent)

//@TODO: CONVERSATION: Assert or otherwise guard all the Server* functions to only execute on the authority

UConversationParticipantComponent::UConversationParticipantComponent()
{
	SetIsReplicatedByDefault(true);
}

void UConversationParticipantComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;
	
	SharedParams.Condition = COND_SkipOwner;
	DOREPLIFETIME_WITH_PARAMS_FAST(UConversationParticipantComponent, ConversationsActive, SharedParams);
}

#if WITH_SERVER_CODE

void UConversationParticipantComponent::ServerNotifyConversationStarted(UConversationInstance* Conversation, FGameplayTag AsParticipant)
{
	AActor* Owner = GetOwner();
	if (Owner->GetLocalRole() == ROLE_Authority)
	{
		check(Conversation);
		Auth_CurrentConversation = Conversation;
		Auth_Conversations.Add(Conversation);

		//@TODO: CONVERSATION: ClientUpdateParticipants, we need to do this immediately so when we tell the client a task has been
		// executed, the client has knowledge of what participants, before any client side task effects need to execute.
		ClientUpdateParticipants(Auth_CurrentConversation->GetParticipantsCopy());

		const int32 OldConversationsActive = ConversationsActive;

		ConversationsActive++;
		MARK_PROPERTY_DIRTY_FROM_NAME(UConversationParticipantComponent, ConversationsActive, this);

		if (OldConversationsActive == 0)
		{
			OnRep_ConversationsActive(OldConversationsActive);
		}

		OnServerConversationStarted(Conversation, AsParticipant);
		ClientStartConversation(Conversation->GetParticipantsCopy());

		ClientUpdateConversations(ConversationsActive);
	}
}

void UConversationParticipantComponent::ServerNotifyConversationEnded(UConversationInstance* Conversation, const FConversationParticipants& PreservedParticipants)
{
	AActor* Owner = GetOwner();
	if (Owner->GetLocalRole() == ROLE_Authority)
	{
		check(Conversation);

		for (UConversationInstance* ConversationInstance : Auth_Conversations)
		{
			if (Conversation == ConversationInstance)
			{
				if (Conversation == Auth_CurrentConversation)
				{
					Auth_CurrentConversation = nullptr;
				}
				
				Auth_Conversations.Remove(Conversation);

				const int32 OldConversationsActive = ConversationsActive;
				ConversationsActive--;
				MARK_PROPERTY_DIRTY_FROM_NAME(UConversationParticipantComponent, ConversationsActive, this);

				OnServerConversationEnded(Conversation);
				ClientExitConversation(PreservedParticipants);

				ClientUpdateConversations(ConversationsActive);

				if (ConversationsActive == 0)
				{
					OnRep_ConversationsActive(OldConversationsActive);
				}

				break;
			}
		}
	}
}

void UConversationParticipantComponent::ServerNotifyExecuteTaskAndSideEffects(const FConversationNodeHandle& Handle, const UConversationDatabase* Graph)
{
	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		ClientExecuteTaskAndSideEffects(Handle, Graph);
	}
}

void UConversationParticipantComponent::ServerForAllConversationsRefreshChoices(UConversationInstance* IgnoreConversation)
{
	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		for (UConversationInstance* Conversation : Auth_Conversations)
		{
			if (Conversation == IgnoreConversation)
			{
				continue;
			}

			Conversation->ServerRefreshConversationChoices();
		}
	}
}

void UConversationParticipantComponent::ServerGetReadyToConverse()
{
}

#endif

FText UConversationParticipantComponent::GetParticipantDisplayName()
{
	if (AActor* Owner = GetOwner())
	{
		return FText::FromString(Owner->GetHumanReadableName());
	}
	return FText();
}

FConversationNodeHandle UConversationParticipantComponent::GetCurrentNodeHandle() const
{
	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		if (Auth_CurrentConversation)
		{
			return Auth_CurrentConversation->GetCurrentNodeHandle();
		}
	}
	else
	{
		return LastMessage.CurrentNode;
	}

	return FConversationNodeHandle();
}

const FConversationParticipantEntry* UConversationParticipantComponent::GetParticipant(const FGameplayTag& ParticipantTag) const
{
	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		if (Auth_CurrentConversation)
		{
			return Auth_CurrentConversation->GetParticipant(ParticipantTag);
		}
	}
	else
	{
		return LastMessage.Participants.GetParticipant(ParticipantTag);
	}

	return nullptr;
}

AActor* UConversationParticipantComponent::GetParticipantActor(const FGameplayTag& ParticipantTag) const
{
	if (const FConversationParticipantEntry* const ParticipantEntry = GetParticipant(ParticipantTag))
	{
		return ParticipantEntry->Actor;
	}

	return nullptr;
}

void UConversationParticipantComponent::GetOtherParticipantActors(TArray<AActor*>& OutParticipantActors) const
{
	const TArray<FConversationParticipantEntry>* ConversationParticipantsPtr = nullptr;

	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		if (Auth_CurrentConversation)
		{
			ConversationParticipantsPtr = &Auth_CurrentConversation->GetParticipantList();
		}
	}
	else
	{
		ConversationParticipantsPtr = &LastMessage.Participants.List;
	}

	if (ConversationParticipantsPtr == nullptr)
	{
		return;
	}

	for (const FConversationParticipantEntry& ConversationParticipantEntry : *ConversationParticipantsPtr)
	{
		if (ConversationParticipantEntry.GetParticipantComponent() == this)
		{
			// Ignore self
			continue;
		}

		if (!IsValid(ConversationParticipantEntry.Actor))
		{
			continue;
		}

		OutParticipantActors.Add(ConversationParticipantEntry.Actor);
	}
}

void UConversationParticipantComponent::SendClientConversationMessage(const FConversationContext& Context, const FClientConversationMessagePayload& Payload)
{
#if WITH_SERVER_CODE
	LastMessage = Payload;

	//@TODO: CONVERSATION: We could potentially send the user no choices?  I guess that's a possibility.
	ClientUpdateConversation(LastMessage);
#endif
}

void UConversationParticipantComponent::ClientUpdateConversations_Implementation(int32 InConversationsActive)
{
	const int32 OldConversationsActive = ConversationsActive;
	ConversationsActive = InConversationsActive;
	OnRep_ConversationsActive(OldConversationsActive);
}

void UConversationParticipantComponent::SendClientUpdatedChoices(const FConversationContext& Context, bool bForcedRefresh)
{
#if WITH_SERVER_CODE
	const TArray<FClientConversationOptionEntry> NewOptions = Context.GetActiveConversation()->GetCurrentUserConversationChoices();
	if (NewOptions != LastMessage.Options || bForcedRefresh)
	{
		LastMessage.Options = NewOptions;
		ClientUpdateConversation(LastMessage);
	}
#endif
}

void UConversationParticipantComponent::SendClientRefreshedTaskChoiceData(const FConversationNodeHandle& Handle, const FConversationContext& Context)
{
#if WITH_SERVER_CODE
	const TArray<FClientConversationOptionEntry> CurrentOptions = Context.GetActiveConversation()->GetCurrentUserConversationChoices();
	
	for (const FClientConversationOptionEntry& CurrentOption : CurrentOptions)
	{
		if (CurrentOption.ChoiceReference.NodeReference == Handle)
		{
			for (FClientConversationOptionEntry& LastOption : LastMessage.Options)
			{
				// UConversationInstance::FindBranchPointFromClientChoice will reject advancing a conversation from a client choice
				// if InBranchPoint.ClientChoice == InChoice. This equality operation requires the client have the most up to date
				// values for all fields. If *any* of these have changed, we should allow refresh. 
				// If required we could eventually just send the entire FClientConversationOptionEntry here
				// and process it similarly in ClientUpdateConversationTaskChoiceData_Implementation, but attempting
				// to be somewhat selective in processing all 'known dynamic fields' while still possible for now to reduce overheads
				if (LastOption.ChoiceReference.NodeReference == Handle && 
					(LastOption.ExtraData != CurrentOption.ExtraData || 
					LastOption.ChoiceReference.NodeParameters != CurrentOption.ChoiceReference.NodeParameters))
				{
					LastOption.ExtraData = CurrentOption.ExtraData;
					LastOption.ChoiceReference.NodeParameters = CurrentOption.ChoiceReference.NodeParameters;
					ClientUpdateConversationTaskChoiceData(Handle, LastOption);
					break;
				}
			}
		}
	}
#endif
}

void UConversationParticipantComponent::ClientUpdateConversation_Implementation(const FClientConversationMessagePayload& Message)
{
	++MessageIndex;
	LastMessage = Message;

	UE_LOGF(LogCommonConversationRuntime, Log, "ClientUpdateConversation: %ls", *Message.ToString());

	OnConversationUpdated(LastMessage);

	if (!bIsFirstConversationUpdateBroadcasted)
	{
		bIsFirstConversationUpdateBroadcasted = true;
	}
	ConversationUpdated.Broadcast(LastMessage);
}

void UConversationParticipantComponent::ClientUpdateConversationTaskChoiceData_Implementation(FConversationNodeHandle Handle, const FClientConversationOptionEntry& OptionEntry)
{
	UE_LOGF(LogCommonConversationRuntime, Log, "ClientUpdateConversationTaskChoiceData :%ls", *Handle.ToString());

	for (FClientConversationOptionEntry& ExistingOption : LastMessage.Options)
	{
		if (ExistingOption.ChoiceReference.NodeReference == Handle)
		{
			ExistingOption.ExtraData = OptionEntry.ExtraData;
			ExistingOption.ChoiceReference.NodeParameters = OptionEntry.ChoiceReference.NodeParameters;
			ConversationTaskChoiceDataUpdated.Broadcast(Handle, OptionEntry);
			break;
		}
	}
}

void UConversationParticipantComponent::ClientStartConversation_Implementation(const FConversationParticipants& InParticipants)
{
	bIsFirstConversationUpdateBroadcasted = false;
	ConversationStarted.Broadcast();

	OnClientStartConversation(InParticipants);
}

void UConversationParticipantComponent::ClientExitConversation_Implementation(const FConversationParticipants& InParticipants)
{
	OnClientExitConversation(InParticipants);
}

bool UConversationParticipantComponent::IsInActiveConversation() const
{
	if (Auth_CurrentConversation)
	{
		return true;
	}

	return ConversationsActive > 0;
}

void UConversationParticipantComponent::RequestServerAdvanceConversation(const FAdvanceConversationRequest& InChoicePicked)
{
	// Notify the server to advance the conversation.
	ServerAdvanceConversation(InChoicePicked);
}

void UConversationParticipantComponent::ServerAdvanceConversation_Implementation(const FAdvanceConversationRequest& InChoicePicked)
{
#if WITH_SERVER_CODE
	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		if (Auth_CurrentConversation)
		{
			Auth_CurrentConversation->ServerAdvanceConversation(InChoicePicked);
		}
	}
#endif
}

void UConversationParticipantComponent::ClientUpdateParticipants_Implementation(const FConversationParticipants& InParticipants)
{
	LastMessage.Participants = InParticipants;
}

void UConversationParticipantComponent::ClientExecuteTaskAndSideEffects_Implementation(FConversationNodeHandle Handle, const UConversationDatabase* Graph)
{
	if (const UConversationTaskNode* TaskNode = Cast<UConversationTaskNode>(Handle.TryToResolve_Slow(GetWorld(), Graph)))
	{
		FConversationContext ClientContext = FConversationContext::CreateClientContext(this, TaskNode);
		TaskNode->ExecuteTaskNodeWithSideEffects(ClientContext);
	}
}

void UConversationParticipantComponent::OnRep_ConversationsActive(int32 OldConversationsActive)
{
	const bool bWasConversing = OldConversationsActive > 0;
	const bool bIsConversing = ConversationsActive > 0;

	if (!bWasConversing && bIsConversing)
	{
		OnEnterConversationState();
		ConversationStatusChanged.Broadcast(true);
	}
	else if (bWasConversing && !bIsConversing)
	{
		OnLeaveConversationState();
		ConversationStatusChanged.Broadcast(false);
	}
}

void UConversationParticipantComponent::OnEnterConversationState()
{
	UE_LOGF(LogCommonConversationRuntime, Verbose, "[%ls]: %ls has entered the conversation state.", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(GetOwner()));
}

#if WITH_SERVER_CODE

void UConversationParticipantComponent::OnServerConversationStarted(UConversationInstance* Conversation, FGameplayTag AsParticipant)
{

}

void UConversationParticipantComponent::OnServerConversationEnded(UConversationInstance* Conversation)
{

}

#endif

void UConversationParticipantComponent::OnLeaveConversationState()
{
	UE_LOGF(LogCommonConversationRuntime, Verbose, "[%ls]: %ls has exited the conversation state.", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(GetOwner()));
}

void UConversationParticipantComponent::OnConversationUpdated(const FClientConversationMessagePayload& Message)
{

}

void UConversationParticipantComponent::OnClientStartConversation(const FConversationParticipants& InParticipants)
{

}

void UConversationParticipantComponent::OnClientExitConversation(const FConversationParticipants& InParticipants)
{

}

#if WITH_SERVER_CODE

void UConversationParticipantComponent::ServerAbortAllConversations()
{
	TArray<UConversationInstance*> Conversations = Auth_Conversations;
	for (UConversationInstance* Conversation : Conversations)
	{
		Conversation->ServerAbortConversation();
	}
}

#endif

#if WITH_SERVER_CODE
void UConversationParticipantComponent::ServerForAllConversationsRefreshTaskChoiceData(const FConversationNodeHandle& Handle, UConversationInstance* IgnoreConversation /*= nullptr*/)
{
	if (GetOwner()->GetLocalRole() == ROLE_Authority)
	{
		for (UConversationInstance* Conversation : Auth_Conversations)
		{
			if (Conversation == IgnoreConversation)
			{
				continue;
			}

			Conversation->ServerRefreshTaskChoiceData(Handle);
		}
	}
}
#endif

