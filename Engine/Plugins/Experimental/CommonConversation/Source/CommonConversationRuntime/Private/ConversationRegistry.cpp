// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "CommonConversationRuntimeLogging.h"
#include "ConversationDatabase.h"
#include "Engine/AssetManager.h"
#include "UObject/UObjectIterator.h"
#include "ConversationContext.h"
#include "ConversationInstance.h"
#include "Engine/StreamableManager.h"
#include "GameFeaturesSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationRegistry)

//======================================================================================

TSharedPtr<FConversationsHandle> FConversationsHandle::Create(const UConversationRegistry* InOwningRegistry, const TSharedPtr<FStreamableHandle>& InStreamableHandle, const TArray<FGameplayTag>& InEntryTags)
{
	TSharedPtr<FConversationsHandle> ConversationsHandle = MakeShared<FConversationsHandle>(FPrivateToken{}, const_cast<UConversationRegistry*>(InOwningRegistry), InStreamableHandle, InEntryTags);
	ConversationsHandle->Initialize();

	return ConversationsHandle;
}

FConversationsHandle::FConversationsHandle(FPrivateToken, UConversationRegistry* InOwningRegistry, const TSharedPtr<FStreamableHandle>& InStreamableHandle, const TArray<FGameplayTag>& InEntryTags)
	: StreamableHandle(InStreamableHandle)
	, ConversationEntryTags(InEntryTags)
	, OwningRegistryPtr(InOwningRegistry)
{
}

void FConversationsHandle::Initialize()
{
	if (UConversationRegistry* OwningRegistry = OwningRegistryPtr.Get())
	{
		OwningRegistry->AvailableConversationsChanged.AddSP(this, &FConversationsHandle::HandleAvailableConversationsChanged);
	}
}

void FConversationsHandle::HandleAvailableConversationsChanged()
{
	if (const UConversationRegistry* OwningRegistry = OwningRegistryPtr.Get())
	{
		// Requery the conversations, and swap the streamable handle out.
		TSharedPtr<FConversationsHandle> TempConversation = OwningRegistry->LoadConversationsFor(ConversationEntryTags);
		StreamableHandle = TempConversation->StreamableHandle;
	}
}

//======================================================================================

void FNetSerializeScriptStructCache_ConvVersion::InitForType(UScriptStruct* InScriptStruct)
{
	ScriptStructsToIndex.Reset();
	IndexToScriptStructs.Reset();

	// Find all script structs of this type and add them to the list
	// (not sure of a better way to do this but it should only happen once at startup)
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		if (It->IsChildOf(InScriptStruct))
		{
			IndexToScriptStructs.Add(*It);
		}
	}

	IndexToScriptStructs.Sort([](const UScriptStruct& A, const UScriptStruct& B) { return A.GetName().ToLower() > B.GetName().ToLower(); });

	for (int Index = 0; Index < IndexToScriptStructs.Num(); Index++)
	{
		ScriptStructsToIndex.Add(IndexToScriptStructs[Index], Index);
	}
}

bool FNetSerializeScriptStructCache_ConvVersion::NetSerialize(FArchive& Ar, UScriptStruct*& Struct)
{
	if (Ar.IsSaving())
	{
		if (int32* IndexPtr = ScriptStructsToIndex.Find(Struct))
		{
			int32 Index = *IndexPtr;
			if (Index <= 127)
			{
				int8 l = (int8)Index;
				Ar.SerializeBits(&l, 8);

			}
			//else
			//{
			//	check(Index <= 32767)

			//	uint8 l = (uint8)(((Index << 24) >> 24) | 128);
			//	uint8 h = (uint8)(Index >> 8);
			//	Ar.SerializeBits(&l, 8);
			//	Ar.SerializeBits(&h, 8);
			//}

			return true;
		}

		UE_LOGF(LogCommonConversationRuntime, Error, "Could not find %ls in ScriptStructCache", *GetNameSafe(Struct));
		return false;
	}
	else
	{
		uint8 Index = 0;
		Ar.SerializeBits(&Index, 8);

		//if (l & 128)
		//{
		//	//int8 h = 0;
		//	//Ar.SerializeBits(&h, 8);
		//}
		//else
		//{
		//	Index = l;
		//}

		if (IndexToScriptStructs.IsValidIndex(Index))
		{
			Struct = IndexToScriptStructs[Index];
			return true;
		}

		UE_LOGF(LogCommonConversationRuntime, Error, "Could not script struct at idx %d", Index);
		return false;
	}
}

//======================================================================================

const UConversationNode* FConversationNodeHandle::TryToResolve_Slow(UWorld* InWorld, const UConversationDatabase* Graph) const
{
	if (UConversationRegistry* Registry = UConversationRegistry::GetFromWorld(InWorld))
	{
		return Registry->TryGetRuntimeNodeFromGUID(NodeGUID, Graph);
	}
	return nullptr;
}

const UConversationNode* FConversationNodeHandle::TryToResolve(const FConversationContext& Context) const
{
	if (UConversationInstance* Instance = Context.TryGetActiveConversation())
	{
		return Context.GetConversationRegistry().TryGetRuntimeNodeFromGUID(NodeGUID, Instance->GetActiveConversationGraph());
	}
	return Context.GetConversationRegistry().TryGetRuntimeNodeFromGUID(NodeGUID);
}

UConversationRegistry::UConversationRegistry()
{
	ConversationChoiceDataStructCache.InitForType(FConversationChoiceData::StaticStruct());
}

UConversationRegistry* UConversationRegistry::GetFromWorld(const UWorld* World)
{
	return UWorld::GetSubsystem<UConversationRegistry>(World);
}

void UConversationRegistry::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UGameFeaturesSubsystem::Get().AddObserver(this, UGameFeaturesSubsystem::EObserverPluginStateUpdateMode::FutureOnly);
	GameFeatureStateModified();
}

void UConversationRegistry::Deinitialize()
{
	if (UGameFeaturesSubsystem* GameFeaturesSubsystem = GEngine ? GEngine->GetEngineSubsystem<UGameFeaturesSubsystem>() : nullptr)
	{
		GameFeaturesSubsystem->RemoveObserver(this);
	}

	Super::Deinitialize();
}

void UConversationRegistry::GameFeatureStateModified()
{
	// If nobody has actually built the dependency graph yet, there's no reason to invalidate anything, nobody is using it yet.
	if (bDependenciesBuilt)
	{
		bDependenciesBuilt = false;
		BuildDependenciesGraph();
	}
}

void UConversationRegistry::OnGameFeatureActivated(const UGameFeatureData* GameFeatureData, const FString& PluginURL)
{
	if (bWaitForDependenciesGraphFlush)
	{
		return;
	}
	GameFeatureStateModified();
}

void UConversationRegistry::OnGameFeatureDeactivating(const UGameFeatureData* GameFeatureData, FGameFeatureDeactivatingContext& Context, const FString& PluginURL)
{
	if (bWaitForDependenciesGraphFlush)
	{
		return;
	}
	GameFeatureStateModified();
}

UConversationNode* UConversationRegistry::GetRuntimeNodeFromGUID(const FGuid& NodeGUID, const UConversationDatabase* Graph) const
{
	// It's possible this is just a null/empty guid, if that happens just return null.
	if (!NodeGUID.IsValid())
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::GetRuntimeNodeFromGUID - NodeGUID not valid");
		return nullptr;
	}

	UConversationNode* Node = TryGetRuntimeNodeFromGUID(NodeGUID, Graph);
	ensureMsgf(Node != nullptr, TEXT("Unexpected GetRuntimeNodeFromGUID(%s) Failed. Nodes Searched: %d"), *NodeGUID.ToString(), NodeGuidToConversation.Num());
	return Node;
}

UConversationNode* UConversationRegistry::TryGetRuntimeNodeFromGUID(const FGuid& NodeGUID, const UConversationDatabase* Graph) const
{
	UE_LOGF(LogCommonConversationRuntime, Verbose, "Start UConversationRegistry::GetRuntimeNodeFromGUID with NodeGUID: (%ls)", *NodeGUID.ToString());

	const_cast<UConversationRegistry*>(this)->BuildDependenciesGraph();

	// It's possible this is just a null/empty guid, if that happens just return null.
	if (!NodeGUID.IsValid())
	{
		return nullptr;
	}

	if (const UConversationDatabase* ConversationDB = Graph ? Graph : GetConversationFromNodeGUID(NodeGUID))
	{
		return ConversationDB->ReachableNodeMap.FindRef(NodeGUID);
	}

	return nullptr;
}

TArray<FGuid> UConversationRegistry::GetEntryPointGUIDs(const FGameplayTag& EntryPoint) const
{
	const_cast<UConversationRegistry*>(this)->BuildDependenciesGraph();

	return EntryTagToEntryList.FindRef(EntryPoint);
}

TArray<FGuid> UConversationRegistry::GetOutputLinkGUIDs(const FGameplayTag& EntryPoint) const
{
	TArray<FGuid> SourceGUIDs = GetEntryPointGUIDs(EntryPoint);
	return GetOutputLinkGUIDs(SourceGUIDs);
}

TArray<FGuid> UConversationRegistry::GetOutputLinkGUIDs(const FGuid& SourceGUID) const
{
	return GetOutputLinkGUIDs(TArray<FGuid>({ SourceGUID }));
}

TArray<FGuid> UConversationRegistry::GetOutputLinkGUIDs(const TArray<FGuid>& SourceGUIDs) const
{
	const_cast<UConversationRegistry*>(this)->BuildDependenciesGraph();

	TArray<FGuid> Result;

	for (const FGuid& SourceGUID : SourceGUIDs)
	{
		if (const UConversationDatabase* SourceConversation = GetConversationFromNodeGUID(SourceGUID))
		{
			UConversationNode* SourceNode = SourceConversation->ReachableNodeMap.FindRef(SourceGUID);
			if (SourceNode)
			{
				if (UConversationNodeWithLinks* SourceNodeWithLinks = CastChecked<UConversationNodeWithLinks>(SourceNode))
				{
					Result.Append(SourceNodeWithLinks->OutputConnections);
				}
			}
		}
	}

	return Result;
}

TArray<FGuid> UConversationRegistry::GetOutputLinkGUIDs(const UConversationDatabase* Graph, const FGameplayTag& EntryPoint, const FString& EntryIdentifier) const
{
	if (Graph == nullptr)
	{
		return GetOutputLinkGUIDs(EntryPoint);
	}

	TArray<FGuid> Result;

	for (const FConversationEntryList& Tag : Graph->EntryTags)
	{
		if (Tag.EntryTag == EntryPoint && Tag.EntryIdentifier == EntryIdentifier)
		{
			for (const FGuid& Destination : Tag.DestinationList)
			{
				UConversationNode* SourceNode = Graph->ReachableNodeMap.FindRef(Destination);
				if (SourceNode)
				{
					if (UConversationNodeWithLinks* SourceNodeWithLinks = CastChecked<UConversationNodeWithLinks>(SourceNode))
					{
						Result.Append(SourceNodeWithLinks->OutputConnections);
					}
				}
			}
			break;
		}
	}

	return Result;
}

TArray<FGuid> UConversationRegistry::GetOutputLinkGUIDs(const UConversationDatabase* Graph, const FGuid& SourceGUID) const
{
	if (Graph == nullptr)
	{
		return GetOutputLinkGUIDs(SourceGUID);
	}

	const_cast<UConversationRegistry*>(this)->BuildDependenciesGraph();

	TArray<FGuid> Result;

	UConversationNode* SourceNode = Graph->ReachableNodeMap.FindRef(SourceGUID);
	if (SourceNode)
	{
		if (UConversationNodeWithLinks* SourceNodeWithLinks = CastChecked<UConversationNodeWithLinks>(SourceNode))
		{
			Result.Append(SourceNodeWithLinks->OutputConnections);
		}
	}

	return Result;
}

void UConversationRegistry::FlushBuildDependenciesGraph()
{
	if (bWaitForDependenciesGraphFlush)
	{
		GameFeatureStateModified();
		bWaitForDependenciesGraphFlush = false;
	}
}

void UConversationRegistry::SetWaitDependenciesGraphToFlush(bool bWaitToFlush)
{
	bWaitForDependenciesGraphFlush = bWaitToFlush;
}

UConversationDatabase* UConversationRegistry::GetConversationFromNodeGUID(const FGuid& NodeGUID) const
{
	if (const FSoftObjectPath* ConversationPathPtr = NodeGuidToConversation.Find(NodeGUID))
	{
		if (UConversationDatabase* ConversationDB = Cast<UConversationDatabase>(ConversationPathPtr->ResolveObject()))
		{
			return ConversationDB;
		}

		if (UConversationDatabase* ConversationDB = Cast<UConversationDatabase>(UAssetManager::GetStreamableManager().LoadSynchronous(*ConversationPathPtr, false)))
		{
			return ConversationDB;
		}
	}

	return nullptr;
}

TSharedPtr<FConversationsHandle> UConversationRegistry::LoadConversationsFor(const FGameplayTag& ConversationEntryTag) const
{
	return LoadConversationsFor(TArray<FGameplayTag>({ ConversationEntryTag }));
}

TSharedPtr<FConversationsHandle> UConversationRegistry::LoadConversationsFor(const TArray<FGameplayTag>& ConversationEntryTags) const
{
	const_cast<UConversationRegistry*>(this)->BuildDependenciesGraph();

	TSet<FSoftObjectPath> ConversationsToLoad;
	for (const FGameplayTag& ConversationEntryTag : ConversationEntryTags)
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::LoadConversationsFor - ConversationEntryTag to find: %ls", *ConversationEntryTag.ToString());

		if (const TArray<FSoftObjectPath>* EntryConversations = EntryTagToConversations.Find(ConversationEntryTag))
		{
			ConversationsToLoad.Append(*EntryConversations);

			UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::LoadConversationsFor - ConversationEntryTag that has been found: %ls", *ConversationEntryTag.ToString());

			for (const FSoftObjectPath& EntryConversation : *EntryConversations)
			{
				GetAllDependenciesForConversation(EntryConversation, OUT ConversationsToLoad);

				UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::LoadConversationsFor - Dependency found: %ls", *EntryConversation.ToString());
			}
		}
	}

	if (ConversationsToLoad.Num() > 0)
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "LoadConversationsFor %ls %ls",
			*FString::JoinBy(ConversationEntryTags, TEXT(", "), [](const FGameplayTag& Tag) { return FString::Printf(TEXT("'%s'"), *Tag.ToString()); }),
			*FString::JoinBy(ConversationsToLoad, TEXT(", "), [](const FSoftObjectPath& SoftObjectPath) { return FString::Printf(TEXT("'%s'"), *SoftObjectPath.ToString()); })
		);

		TSharedPtr<FStreamableHandle> StreamableHandle = UAssetManager::Get().LoadAssetList(ConversationsToLoad.Array());
		return FConversationsHandle::Create(this, StreamableHandle, ConversationEntryTags);
	}
	else
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "LoadConversationsFor %ls - NO CONVERSATIONS FOUND",
			*FString::JoinBy(ConversationEntryTags, TEXT(", "), [](const FGameplayTag& Tag) { return FString::Printf(TEXT("'%s'"), *Tag.ToString()); })
		);
	}

	return FConversationsHandle::Create(this, TSharedPtr<FStreamableHandle>(), ConversationEntryTags);
}

TArray<FPrimaryAssetId> UConversationRegistry::GetPrimaryAssetIdsForEntryPoint(FGameplayTag EntryPoint) const
{
	const_cast<UConversationRegistry*>(this)->BuildDependenciesGraph();

	TArray<FPrimaryAssetId> AssetsWithTheEntryPoint;

	UE_LOGF(LogCommonConversationRuntime, Verbose, "Finding PrimaryAssetIds For EntryPoint [%ls] among %d", *EntryPoint.ToString(), EntryTagToConversations.Num());

	if (const TArray<FSoftObjectPath>* ConversationPaths = EntryTagToConversations.Find(EntryPoint))
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::GetPrimaryAssetIdsForEntryPoint - %d Conversation paths found", ConversationPaths->Num());

		UAssetManager& AssetManager = UAssetManager::Get();
		for (const FSoftObjectPath& ConversationPath : *ConversationPaths)
		{
			UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::GetPrimaryAssetIdsForEntryPoint - Found conversation path: %ls", *ConversationPath.ToString());

			FPrimaryAssetId ConversationAssetId = AssetManager.GetPrimaryAssetIdForPath(ConversationPath);
			if (ensure(ConversationAssetId.IsValid()))
			{
				AssetsWithTheEntryPoint.AddUnique(ConversationAssetId);

				UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::GetPrimaryAssetIdsForEntryPoint - Valid conversation path added to AssetsWithTheEntryPoint: %ls", *ConversationAssetId.PrimaryAssetName.ToString());
			}
			else
			{
				UE_LOGF(LogCommonConversationRuntime, Error, "GetPrimaryAssetIdsForEntryPoint Invalid PrimaryAssetId for %ls", *ConversationPath.ToString());
			}
		}
	}

	UE_LOGF(LogCommonConversationRuntime, Verbose, "For EntryPoint [%ls], Found [%ls]",
		*EntryPoint.ToString(),
		*FString::JoinBy(AssetsWithTheEntryPoint, TEXT(", "), [](const FPrimaryAssetId& PrimaryAssetId) { return PrimaryAssetId.ToString(); })
	);

	return AssetsWithTheEntryPoint;
}

TArray<FGameplayTag> UConversationRegistry::GetLinkedExitConversationEntryTags(const UConversationDatabase* ConversationDatabase) const
{
	if (ConversationDatabase)
	{
		return ConversationDatabase->ExitTags.GetGameplayTagArray();
	}
	return {};
}

void UConversationRegistry::BuildDependenciesGraph()
{
	if (bDependenciesBuilt)
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "UConversationRegistry::BuildDependenciesGraph - Dependencies already built");

		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_UConversationRegistry_BuildDependenciesGraph);
	UE_LOGF(LogCommonConversationRuntime, Verbose, "Registry Building Graph");

	TArray<FAssetData> AllActiveConversationAssets;
	UAssetManager::Get().GetPrimaryAssetDataList(FPrimaryAssetType(UConversationDatabase::StaticClass()->GetFName()), AllActiveConversationAssets);

	// Don't index conversation graphs from inactive game feature plug-ins.
	UGameFeaturesSubsystem::Get().FilterInactivePluginAssets(AllActiveConversationAssets);

	// Lets find out if anything actually changed, we maybe rebuilding things after a game feature added or got removed,
	// as far as conversations are concerned it's possible nothing has changed.
	if (RuntimeDependencyGraph.Num() == AllActiveConversationAssets.Num())
	{
		bool ConversationAssetsChanged = false;
		for (const FAssetData& ConversationDataAsset : AllActiveConversationAssets)
		{
			if (!RuntimeDependencyGraph.Contains(ConversationDataAsset.ToSoftObjectPath()))
			{
				ConversationAssetsChanged = true;
				break;
			}
		}

		// If we need to rebuild the conversation graph, but the conversation graph, wont actually change based on what was
		// loaded or unloaded since the last time BuildDependenciesGraph was called, then we don't actually need to regenerate
		// anything.
		if (!ConversationAssetsChanged)
		{
			bDependenciesBuilt = true;
			return;
		}
	}

	RuntimeDependencyGraph.Reset();
	EntryTagToConversations.Reset();
	EntryTagToEntryList.Reset();
	NodeGuidToConversation.Reset();

	UE_LOGF(LogCommonConversationRuntime, Verbose, "Building: Total Conversations %d, Old Conversations %d", AllActiveConversationAssets.Num(), RuntimeDependencyGraph.Num());

	// Seed
	for (const FAssetData& ConversationDataAsset : AllActiveConversationAssets)
	{
		const FString EntryTagsString = ConversationDataAsset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UConversationDatabase, EntryTags));
		if (!EntryTagsString.IsEmpty())
		{
			TArray<FConversationEntryList> EntryTags;

			FArrayProperty* ArrayProperty = FindFProperty<FArrayProperty>(UConversationDatabase::StaticClass(), GET_MEMBER_NAME_CHECKED(UConversationDatabase, EntryTags));
			ArrayProperty->ImportText_Direct(*EntryTagsString, &EntryTags, nullptr, 0);

			for (const FConversationEntryList& Entry : EntryTags)
			{
				EntryTagToConversations.FindOrAdd(Entry.EntryTag).Add(ConversationDataAsset.ToSoftObjectPath());
				EntryTagToEntryList.FindOrAdd(Entry.EntryTag).Append(Entry.DestinationList);
			}
		}

		const FString InternalNodeIds = ConversationDataAsset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UConversationDatabase, InternalNodeIds));
		if (!InternalNodeIds.IsEmpty())
		{
			TArray<FGuid> NodeIds;

			FArrayProperty* ArrayProperty = FindFProperty<FArrayProperty>(UConversationDatabase::StaticClass(), GET_MEMBER_NAME_CHECKED(UConversationDatabase, InternalNodeIds));
			ArrayProperty->ImportText_Direct(*InternalNodeIds, &NodeIds, nullptr, 0);

			for (FGuid& NodeId : NodeIds)
			{
				if (const FSoftObjectPath* const ExistingEntry = NodeGuidToConversation.Find(NodeId);
					ensureAlwaysMsgf(ExistingEntry == nullptr, TEXT("Trying to register NodeId %s to %s, but it's already registered to %s"), *NodeId.ToString(), *ConversationDataAsset.ToSoftObjectPath().ToString(), *ExistingEntry->ToString()))
				{
					NodeGuidToConversation.Add(NodeId, ConversationDataAsset.ToSoftObjectPath());
				}
			}
		}
	}

	UE_LOGF(LogCommonConversationRuntime, Verbose, "Building: Total Entry Points %d", EntryTagToConversations.Num());
	UE_LOGF(LogCommonConversationRuntime, Verbose, "Building: Total Nodes %d", NodeGuidToConversation.Num());

	for (const FAssetData& ConversationDataAsset : AllActiveConversationAssets)
	{
		const FString ExitTagsString = ConversationDataAsset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UConversationDatabase, ExitTags));
		if (!ExitTagsString.IsEmpty())
		{
			FGameplayTagContainer ExitTags;
			ExitTags.FromExportString(ExitTagsString);

			TArray<FSoftObjectPath>& Conversations = RuntimeDependencyGraph.FindOrAdd(ConversationDataAsset.ToSoftObjectPath());
			for (const FGameplayTag& ExitTag : ExitTags)
			{
				if (TArray<FSoftObjectPath>* Imported = EntryTagToConversations.Find(ExitTag))
				{
					Conversations.Append(*Imported);
				}
			}
		}

		const FString LinkedToNodeIds = ConversationDataAsset.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UConversationDatabase, LinkedToNodeIds));
		if (!LinkedToNodeIds.IsEmpty())
		{
			TArray<FGuid> NodeIds;

			FArrayProperty* ArrayProperty = FindFProperty<FArrayProperty>(UConversationDatabase::StaticClass(), GET_MEMBER_NAME_CHECKED(UConversationDatabase, LinkedToNodeIds));
			ArrayProperty->ImportText_Direct(*LinkedToNodeIds, &NodeIds, nullptr, 0);

			//@TODO: CONVERSATION: Register that we need to link to other graphs here.
		}
	}

	UE_LOGF(LogCommonConversationRuntime, Verbose, "Building: Runtime Dependency Graph");
	for (const auto& KVP : RuntimeDependencyGraph)
	{
		UE_LOGF(LogCommonConversationRuntime, Verbose, "Conversation: %ls", *KVP.Key.ToString());
		for (const FSoftObjectPath& Dependency : KVP.Value)
		{
			UE_LOGF(LogCommonConversationRuntime, Verbose, "     Include: %ls", *Dependency.ToString());
		}
	}

	bDependenciesBuilt = true;

	AvailableConversationsChanged.Broadcast();
}

void UConversationRegistry::GetAllDependenciesForConversation(const FSoftObjectPath& Parent, TSet<FSoftObjectPath>& OutConversationsToLoad) const
{
	if (const TArray<FSoftObjectPath>* Dependencies = RuntimeDependencyGraph.Find(Parent))
	{
		for (const FSoftObjectPath& Dependency : *Dependencies)
		{
			if (!OutConversationsToLoad.Contains(Dependency))
			{
				OutConversationsToLoad.Add(Dependency);
				GetAllDependenciesForConversation(Dependency, OutConversationsToLoad);
			}
		}
	}
}
