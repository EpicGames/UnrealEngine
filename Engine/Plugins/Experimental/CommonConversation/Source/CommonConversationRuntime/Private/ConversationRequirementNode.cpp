// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationRequirementNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationRequirementNode)

EConversationRequirementResult MergeRequirements(EConversationRequirementResult CurrentResult, EConversationRequirementResult MergeResult)
{
	if ((int64)MergeResult > (int64)CurrentResult)
	{
		return MergeResult;
	}

	return CurrentResult;
}

EConversationRequirementResult UConversationRequirementNode::IsRequirementSatisfied_Implementation(const FConversationContext& Context) const
{
	return EConversationRequirementResult::FailedAndHidden;
}

