// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationSettings)

UConversationSettings::UConversationSettings()
{

}
