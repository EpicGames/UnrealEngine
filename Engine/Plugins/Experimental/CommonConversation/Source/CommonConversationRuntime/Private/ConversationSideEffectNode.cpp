// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationSideEffectNode.h"
#include "ConversationContext.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationSideEffectNode)

void UConversationSideEffectNode::CauseSideEffect(const FConversationContext& Context) const
{
	TGuardValue<decltype(EvalWorldContextObj)> Swapper(EvalWorldContextObj, Context.GetWorld());
	
	if (Context.IsServerContext())
	{
		ServerCauseSideEffect(Context);
	}

	if (Context.IsClientContext())
	{
		ClientCauseSideEffect(Context);
	}
}

void UConversationSideEffectNode::ServerCauseSideEffect_Implementation(const FConversationContext& Context) const
{
}

void UConversationSideEffectNode::ClientCauseSideEffect_Implementation(const FConversationContext& Context) const
{
}
