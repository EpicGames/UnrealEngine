// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationTaskNode.h"

#include "CommonConversationRuntimeLogging.h"
#include "ConversationChoiceNode.h"
#include "ConversationContext.h"
#include "ConversationInstance.h"
#include "ConversationRegistry.h"
#include "ConversationRequirementNode.h"
#include "ConversationSideEffectNode.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationTaskNode)

#if WITH_EDITOR
FName UConversationTaskNode::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.Icon");
}
#endif	// WITH_EDITOR

EConversationRequirementResult UConversationTaskNode::CheckRequirements(const FConversationContext& InContext) const
{
	check(InContext.IsServerContext());

	FConversationContext Context = InContext.CreateChildContext(this);
	EConversationRequirementResult FinalRequirementResult = EConversationRequirementResult::Passed;

	UWorld* World = Context.GetWorld();

	// Iterate sub node requirements, before we iterate core task node requirements. 
	// whilst counter-intuitive, task node requirements often have expensive checks (e.g physics tests) which we want to avoid
	// Conversely sub node requirements often feature simple tag gating checks, which we would prefer to fail on and bail quickly

	// Iterate subnodes that are requirements and see if any don't pass.
	for (const UConversationNode* SubNode : SubNodes)
	{
		if (const UConversationRequirementNode* RequirementNode = Cast<UConversationRequirementNode>(SubNode))
		{
			TGuardValue<decltype(RequirementNode->EvalWorldContextObj)> Swapper(RequirementNode->EvalWorldContextObj, World);

			const EConversationRequirementResult RequirementResult = RequirementNode->IsRequirementSatisfied(Context);

			FinalRequirementResult = MergeRequirements(FinalRequirementResult, RequirementResult);

			if (RequirementResult != EConversationRequirementResult::Passed)
			{
				UE_LOGF(LogCommonConversationRuntime, Verbose, "\tRequirement %ls %ls", *GetPathNameSafe(RequirementNode), *StaticEnum<EConversationRequirementResult>()->GetNameStringByValue((int64)RequirementResult));
			}

			if (FinalRequirementResult == EConversationRequirementResult::FailedAndHidden)
			{
				// Can't get any more failed than this
				break;
			}
		}
	}

	if (FinalRequirementResult != EConversationRequirementResult::FailedAndHidden)
	{
		// If this task has innate requirements, we should check those.
		{
			TGuardValue<decltype(EvalWorldContextObj)> Swapper(EvalWorldContextObj, World);

			const EConversationRequirementResult RequirementResult = IsRequirementSatisfied(Context);
			if (RequirementResult != EConversationRequirementResult::Passed)
			{
				UE_LOGF(LogCommonConversationRuntime, Verbose, "\tRequirement %ls %ls", *GetPathNameSafe(this), *StaticEnum<EConversationRequirementResult>()->GetNameStringByValue((int64)RequirementResult));
			}

			FinalRequirementResult = MergeRequirements(FinalRequirementResult, RequirementResult);
		}
	}

	return FinalRequirementResult;
}

EConversationRequirementResult UConversationTaskNode::IsRequirementSatisfied_Implementation(const FConversationContext& Context) const
{
	return EConversationRequirementResult::Passed;
}

FConversationTaskResult UConversationTaskNode::ExecuteTaskNode_Implementation(const FConversationContext& Context) const
{
	return FConversationTaskResult::AdvanceConversation();
}

void UConversationTaskNode::ExecuteClientEffects_Implementation(const FConversationContext& Context) const
{

}

bool UConversationTaskNode::GetNodeBodyColor_Implementation(FLinearColor& BodyColor) const
{
#if WITH_EDITORONLY_DATA
	BodyColor = DefaultNodeBodyColor;
#else
	BodyColor = FLinearColor::White;
#endif
	return false;
}

FConversationTaskResult UConversationTaskNode::ExecuteTaskNodeWithSideEffects(const FConversationContext& InContext) const
{
	ensure(InContext.GetTaskBeingConsidered() == this);

	TGuardValue<decltype(EvalWorldContextObj)> Swapper(EvalWorldContextObj, InContext.GetWorld());

	FConversationTaskResult Result(ForceInit);

#if WITH_SERVER_CODE
	if (InContext.IsServerContext())
	{
		Result = ExecuteTaskNode(InContext);
		ensureMsgf(Result.GetType() != EConversationTaskResultType::Invalid, TEXT("Conversation Node %s - Returned an Invalid result indicating no specific decision was made on how to continue."), *GetName());

		// After executing the task we need to determine if we should run side effects on the server and client.
		if (Result.CanConversationContinue())
		{
			for (UConversationNode* SubNode : SubNodes)
			{
				if (UConversationSideEffectNode* SideEffectNode = Cast<UConversationSideEffectNode>(SubNode))
				{
					UE_LOGF(LogCommonConversationRuntime, Verbose, "\tRunning side effect %ls", *GetPathNameSafe(SideEffectNode));
					SideEffectNode->CauseSideEffect(InContext);
				}
			}

			const UConversationDatabase* Graph = nullptr;
			if (const UConversationInstance* Conversation = InContext.GetActiveConversation())
			{
				Graph = Conversation->GetActiveConversationGraph();
			}

			FConversationParticipants Participants = InContext.GetParticipantsCopy();
			for (const FConversationParticipantEntry& ParticipantEntry : Participants.List)
			{
				if (UConversationParticipantComponent* Component = ParticipantEntry.GetParticipantComponent())
				{
					// Notify each client in the conversation
					Component->ServerNotifyExecuteTaskAndSideEffects(InContext.GetCurrentNodeHandle(), Graph);
				}
			}
		}
	}
#endif

	if (InContext.IsClientContext())
	{
		ExecuteClientEffects(InContext);

		for (UConversationSubNode* SubNode : SubNodes)
		{
			if (UConversationSideEffectNode* SideEffect = Cast<UConversationSideEffectNode>(SubNode))
			{
				SideEffect->CauseSideEffect(InContext);
			}
		}
	}

	return Result;
}

void UConversationTaskNode::GenerateChoicesForDestinations(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& InContext, const TArray<FGuid>& CandidateDestinations)
{
	check(InContext.IsServerContext());

	UWorld* World = InContext.GetWorld();
	const UConversationDatabase* Graph = nullptr;
	if (const UConversationInstance* Conversation = InContext.GetActiveConversation())
	{
		Graph = Conversation->GetActiveConversationGraph();
	}

	for (const FGuid& DestinationGUID : CandidateDestinations)
	{
		if (UConversationTaskNode* DestinationTaskNode = Cast<UConversationTaskNode>(InContext.GetConversationRegistry().GetRuntimeNodeFromGUID(DestinationGUID, Graph)))
		{
			TGuardValue<decltype(DestinationTaskNode->EvalWorldContextObj)> Swapper(DestinationTaskNode->EvalWorldContextObj, World);

			FConversationContext DestinationContext = InContext.CreateChildContext(DestinationTaskNode);

			const int32 StartingNumber = BranchBuilder.Num();

			DestinationTaskNode->GatherChoices(BranchBuilder, DestinationContext);

			// If a node has no choices, but we're generating the choices, we need to have this node as 'a' choice, even if
			// it's not something we're ever sending to the client, we just need to know this is a valid path for the
			// conversation to flow.
			if (BranchBuilder.Num() == StartingNumber)
			{
				const EConversationRequirementResult RequirementResult = DestinationTaskNode->CheckRequirements(InContext);

				if (RequirementResult == EConversationRequirementResult::Passed)
				{
					FClientConversationOptionEntry DefaultChoice;
					DefaultChoice.ChoiceReference.NodeReference = DestinationGUID;
					DefaultChoice.ChoiceType = EConversationChoiceType::ServerOnly;
					BranchBuilder.AddChoice(DestinationContext, MoveTemp(DefaultChoice));
				}
			}
		}
	}
}

void UConversationTaskNode::GenerateNextChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const
{
	if (const UConversationInstance* Conversation = Context.GetActiveConversation())
	{
		const TArray<FGuid> CandidateDestinations = Context.GetConversationRegistry().GetOutputLinkGUIDs(Conversation->GetActiveConversationGraph(), { Conversation->GetCurrentChoiceReference().NodeReference.NodeGUID });

		UConversationTaskNode::GenerateChoicesForDestinations(BranchBuilder, Context, CandidateDestinations);
	}
}

void UConversationTaskNode::GatherChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const
{
	GatherStaticChoices(BranchBuilder, Context);
	GatherDynamicChoices(BranchBuilder, Context);
}

void UConversationTaskNode::GatherStaticChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& InContext) const
{
	UWorld* World = InContext.GetWorld();

	const EConversationRequirementResult RequirementResult = CheckRequirements(InContext);

	if (RequirementResult < EConversationRequirementResult::FailedAndHidden)
	{
		for (UConversationNode* DestinationSubNode : SubNodes)
		{
			if (UConversationChoiceNode* DestinationChoiceNode = Cast<UConversationChoiceNode>(DestinationSubNode))
			{
				TGuardValue<decltype(DestinationChoiceNode->EvalWorldContextObj)> DestinationSwapper(DestinationChoiceNode->EvalWorldContextObj, World);

				FClientConversationOptionEntry Choice;
				Choice.SetChoiceAvailable(RequirementResult == EConversationRequirementResult::Passed);
				if (DestinationChoiceNode->GenerateChoice(InContext, Choice))
				{
					//@TODO: CONVERSATION: Not a fan of this, would prefer some kinda better system for resolving dynamic vs.
					// static choices and how they get their extra data we communicate to the client.
					GatherStaticExtraData(InContext, Choice.ExtraData);

					BranchBuilder.AddChoice(InContext, MoveTemp(Choice));
				}
			}
		}
	}
}

void UConversationTaskNode::GatherStaticExtraData_Implementation(const FConversationContext& Context, TArray<FConversationNodeParameterPair>& InOutExtraData) const
{
	
}

void UConversationTaskNode::GatherDynamicChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& InContext) const
{

}
