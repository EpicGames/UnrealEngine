// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversationTypes.h"
#include "ConversationContext.h"
#include "ConversationTaskNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConversationTypes)

//////////////////////////////////////////////////////////////////////

struct FConversationChoiceDataDeleter
{
	FORCEINLINE void operator()(FConversationChoiceData* Object) const
	{
		check(Object);
		UScriptStruct* ScriptStruct = Object->GetScriptStruct();
		check(ScriptStruct);
		ScriptStruct->DestroyStruct(Object);
		FMemory::Free(Object);
	}
};

//////////////////////////////////////////////////////////////////////

bool FConversationChoiceDataHandle::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	return false;

	//uint8 DataNum;
	//if (Ar.IsSaving())
	//{
	//	UE_CLOGF(Data.Num() > MAX_uint8, LogCommonConversationRuntime, Warning, "Too many TargetData sources (%d!) to net serialize. Clamping to %d", Data.Num(), MAX_uint8);
	//	DataNum = FMath::Min<int32>(Data.Num(), MAX_uint8);
	//}
	//
	//Ar << DataNum;

	//if (Ar.IsLoading())
	//{
	//	Data.SetNum(DataNum);
	//	if (DataNum > 32)
	//	{
	//		UE_LOGF(LogCommonConversationRuntime, Warning, "FConversationChoiceDataHandle::NetSerialize received with large DataNum: %d", DataNum);
	//	}
	//}

	//for (int32 i = 0; i < DataNum && !Ar.IsError(); ++i)
	//{
	//	TCheckedObjPtr<UScriptStruct> ScriptStruct = Data[i].IsValid() ? Data[i]->GetScriptStruct() : nullptr;

	//	UConversationRegistry::Get().ConversationChoiceDataStructCache.NetSerialize(Ar, ScriptStruct.Get());

	//	if (ScriptStruct.IsValid())
	//	{
	//		if (Ar.IsLoading())
	//		{
	//			// For now, just always reset/reallocate the data when loading.
	//			// Longer term if we want to generalize this and use it for property replication, we should support
	//			// only reallocating when necessary
	//			check(!Data[i].IsValid());

	//			FConversationChoiceData* NewData = (FConversationChoiceData*)FMemory::Malloc(ScriptStruct->GetStructureSize());
	//			ScriptStruct->InitializeStruct(NewData);

	//			Data[i] = TSharedPtr<FConversationChoiceData>(NewData, FConversationChoiceDataDeleter());
	//		}

	//		void* ContainerPtr = Data[i].Get();

	//		if (ScriptStruct->StructFlags & STRUCT_NetSerializeNative)
	//		{
	//			ScriptStruct->GetCppStructOps()->NetSerialize(Ar, Map, bOutSuccess, Data[i].Get());
	//		}
	//		else
	//		{
	//			// This won't work since FStructProperty::NetSerializeItem is deprecrated.
	//			//	1) we have to manually crawl through the topmost struct's fields since we don't have a FStructProperty for it (just the UScriptProperty)
	//			//	2) if there are any UStructProperties in the topmost struct's fields, we will assert in FStructProperty::NetSerializeItem.

	//			UE_LOGF(LogCommonConversationRuntime, Fatal, "FConversationChoiceDataHandle::NetSerialize called on data struct %ls without a native NetSerialize", *ScriptStruct->GetName());

	//			for (TFieldIterator<FProperty> It(ScriptStruct.Get()); It; ++It)
	//			{
	//				if (It->PropertyFlags & CPF_RepSkip)
	//				{
	//					continue;
	//				}

	//				void* PropertyData = It->ContainerPtrToValuePtr<void*>(ContainerPtr);

	//				It->NetSerializeItem(Ar, Map, PropertyData);
	//			}
	//		}
	//	}
	//	else if (ScriptStruct.IsError())
	//	{
	//		UE_LOGF(LogCommonConversationRuntime, Error, "FConversationChoiceDataHandle::NetSerialize: Bad ScriptStruct serialized, can't recover.");
	//		Ar.SetError();
	//		break;
	//	}
	//}

	//if (Ar.IsError())
	//{
	//	// Something bad happened, make sure to not return invalid shared ptrs
	//	for (int32 i = Data.Num() - 1; i >= 0; --i)
	//	{
	//		if (Data[i].IsValid() == false)
	//		{
	//			Data.RemoveAt(i);
	//		}
	//	}
	//	bOutSuccess = false;
	//	return false;
	//}

	//bOutSuccess = true;
	//return true;
}

//////////////////////////////////////////////////////////////////////

static FString LexToString(const TArray<FConversationNodeParameterPair>& InParameters)
{
	if (InParameters.Num() > 0)
	{
		TStringBuilder<256> MapString;
		MapString.Append(TEXT("{ "));
		bool bFirst = true;
		for (const FConversationNodeParameterPair& Entry : InParameters)
		{
			if (!bFirst)
			{
				MapString.Append(TEXT(", "));
			}

			bFirst = false;

			MapString.Append(Entry.Name);
			MapString.Append(TEXT(": "));
			MapString.Append(Entry.Value);
		}
		MapString.Append(TEXT("}"));

		return MapString.ToString();
	}

	return FString();
}

const FConversationChoiceReference FConversationChoiceReference::Empty = FConversationChoiceReference();

FString FConversationChoiceReference::ToString() const
{
	return FString::Printf(TEXT("(Node=%s, Params=%s)"), *NodeReference.ToString(), *LexToString(NodeParameters));
}

//////////////////////////////////////////////////////////////////////

const FAdvanceConversationRequest FAdvanceConversationRequest::Any = FAdvanceConversationRequest();

FString FAdvanceConversationRequest::ToString() const
{
	return FString::Printf(TEXT("(Choice=%s, UserParams=%s)"), *Choice.ToString(), *LexToString(UserParameters));
}

//////////////////////////////////////////////////////////////////////

FAdvanceConversationRequest FClientConversationOptionEntry::ToAdvanceConversationRequest(const TArray<FConversationNodeParameterPair>& InUserParameters) const
{
	FAdvanceConversationRequest Request;
	Request.Choice = ChoiceReference;
	Request.UserParameters = InUserParameters;

	return Request;
}

//////////////////////////////////////////////////////////////////////

void FConversationBranchPointBuilder::AddChoice(const FConversationContext& InContext, FClientConversationOptionEntry&& InChoice)
{
	FConversationBranchPoint BranchPoint;
	BranchPoint.ClientChoice = MoveTemp(InChoice);
	if (!BranchPoint.ClientChoice.ChoiceReference.NodeReference.IsValid())
	{
		BranchPoint.ClientChoice.ChoiceReference.NodeReference = InContext.GetTaskBeingConsidered()->GetNodeGuid();
	}
	BranchPoint.ReturnScopeStack = InContext.GetReturnScopeStack();
	
	BranchPoints.Add(MoveTemp(BranchPoint));
}

//////////////////////////////////////////////////////////////////////

const FConversationParticipantEntry* FConversationParticipants::GetParticipant(FGameplayTag ParticipantID) const
{
	for (const FConversationParticipantEntry& Entry : List)
	{
		if (Entry.ParticipantID == ParticipantID)
		{
			return &Entry;
		}
	}
	return nullptr;
}

UConversationParticipantComponent* FConversationParticipants::GetParticipantComponent(FGameplayTag ParticipantID) const
{
	if (const FConversationParticipantEntry* Entry = GetParticipant(ParticipantID))
	{
		return Entry->GetParticipantComponent();
	}
	return nullptr;
}

bool FConversationParticipants::Contains(AActor* PotentialParticipant) const
{
	for (const FConversationParticipantEntry& Entry : List)
	{
		if (Entry.Actor == PotentialParticipant)
		{
			return true;
		}
	}

	return false;
}

