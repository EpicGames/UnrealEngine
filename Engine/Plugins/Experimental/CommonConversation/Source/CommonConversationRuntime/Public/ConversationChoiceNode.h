// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationSubNode.h"
#include "GameplayTagContainer.h"

#include "ConversationChoiceNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

struct FClientConversationOptionEntry;

/**
 * A choice on a task indicates that an option be presented to the user when the owning task is one of
 * the available options of a preceding task.
 */
UCLASS(MinimalAPI, Blueprintable)
class UConversationChoiceNode : public UConversationSubNode
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, EditAnywhere, meta=(ExposeOnSpawn), Category=Conversation)
	FText DefaultChoiceDisplayText;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Conversation)
	FGameplayTagContainer ChoiceTags;

	bool GetHideChoiceClassName() const { return bHideChoiceClassName; }

	UE_API virtual bool GenerateChoice(const FConversationContext& Context, FClientConversationOptionEntry& ChoiceEntry) const;

	UE_API virtual void NotifyChoicePickedByUser(const FConversationContext& InContext, const FClientConversationOptionEntry& InClientChoice) const;

protected:
	UFUNCTION(BlueprintNativeEvent)
	UE_API void FillChoice(const FConversationContext& Context, FClientConversationOptionEntry& ChoiceEntry) const;

	bool bHideChoiceClassName = false;
};

#undef UE_API
