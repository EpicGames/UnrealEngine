// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "ConversationParticipantComponent.h"

#include "ConversationContext.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

class UConversationInstance;
class UConversationNode;
class UConversationTaskNode;
class UConversationParticipantComponent;
class UConversationRegistry;
class UWorld;

//////////////////////////////////////////////////////////////////////

/**
 * The conversation task result type gives the conversation system the instruction it needs
 * after running a task.  Should we continue to the next task? or stop and give the player
 * the choice of moving forward?
 */
UENUM(BlueprintType)
enum class EConversationTaskResultType : uint8
{
	Invalid,
	/** Aborts the conversation. */
	AbortConversation,
	/** Advances the conversation to the next task, or a random one if there are multiple. */
	AdvanceConversation,
	/**
	 * Advances the conversation to a choice, this choice does not have to be one that would normally come next.
	 * Consider using this in advanced situations where you want to potentially dynamically jump to any node in
	 * existence.
	 */
	AdvanceConversationWithChoice,
	/**
	 * Stops the conversation flow and notifies the client that there are choices, with a payload of anything
	 * the NPC needs to say along with whatever choices the user has.
	 */
	PauseConversationAndSendClientChoices,
	/**
	 * Dynamically allows jumping 'back' one step in the conversation.  This does not go back one Task, but
	 * to the last time in the conversation flow we paused conversation and sent the client choices.
	 */
	ReturnToLastClientChoice,
	/**
	 * Does not advance the conversation, just refreshes the current choices again. 
	 * This option is really useful if you need to have the user make a choice and then
	 * make the same choice again, ex. User clicks an option to buy an item, and you want
	 * them to be able to repeat that action.
	 */
	ReturnToCurrentClientChoice,
	/**
	 * Allows jumping back to the beginning of the entire conversation tree, so that you can effectively, return
	 * to the 'main menu'.
	 */
	ReturnToConversationStart
};

/**
 * The FConversationTaskResult encompasses the type of result along with any extra data we need for
 * that kind of result, for example if we're giving the player a message and giving them a choice, what
 * what message do we need to send.
 */
USTRUCT(BlueprintType)
struct FConversationTaskResult
{
	GENERATED_BODY()

public:
	static FConversationTaskResult AbortConversation()
	{
		return FConversationTaskResult(EConversationTaskResultType::AbortConversation, FAdvanceConversationRequest(), FClientConversationMessage());
	}

	static FConversationTaskResult AdvanceConversation()
	{
		return FConversationTaskResult(EConversationTaskResultType::AdvanceConversation, FAdvanceConversationRequest(), FClientConversationMessage());
	}

	static FConversationTaskResult AdvanceConversationWithChoice(const FAdvanceConversationRequest& InAdvanceToChoice)
	{
		return FConversationTaskResult(EConversationTaskResultType::AdvanceConversationWithChoice, InAdvanceToChoice, FClientConversationMessage());
	}

	static FConversationTaskResult PauseConversationAndSendClientChoices(const FClientConversationMessage& InMessage)
	{
		return FConversationTaskResult(EConversationTaskResultType::PauseConversationAndSendClientChoices, FAdvanceConversationRequest(), InMessage);
	}

	static FConversationTaskResult ReturnToLastClientChoice()
	{
		return FConversationTaskResult(EConversationTaskResultType::ReturnToLastClientChoice, FAdvanceConversationRequest(), FClientConversationMessage());
	}

	static FConversationTaskResult ReturnToCurrentClientChoice()
	{
		return FConversationTaskResult(EConversationTaskResultType::ReturnToCurrentClientChoice, FAdvanceConversationRequest(), FClientConversationMessage());
	}

	static FConversationTaskResult ReturnToConversationStart()
	{
		return FConversationTaskResult(EConversationTaskResultType::ReturnToConversationStart, FAdvanceConversationRequest(), FClientConversationMessage());
	}

	EConversationTaskResultType GetType() const { return Type; }
	const FAdvanceConversationRequest& GetChoice() const { ensure(Type == EConversationTaskResultType::AdvanceConversationWithChoice); return AdvanceToChoice; }
	const FClientConversationMessage& GetMessage() const { ensure(Type == EConversationTaskResultType::PauseConversationAndSendClientChoices); return Message; }

	bool CanConversationContinue() const
	{
		return !(Type == EConversationTaskResultType::Invalid || Type == EConversationTaskResultType::AbortConversation);
	}

public:
	FConversationTaskResult() : Type(EConversationTaskResultType::Invalid) { }

	/**
	 * Constructor
	 *
	 * @param EForceInit Force init enum
	 */
	explicit inline FConversationTaskResult(EForceInit)
		: Type(EConversationTaskResultType::Invalid)
	{
	}

private:
	FConversationTaskResult(EConversationTaskResultType InType, const FAdvanceConversationRequest& InAdvanceToChoice, const FClientConversationMessage& InMessage)
		: Type(InType)
		, AdvanceToChoice(InAdvanceToChoice)
		, Message(InMessage)
	{
	}

	UPROPERTY()
	EConversationTaskResultType Type;

	UPROPERTY()
	FAdvanceConversationRequest AdvanceToChoice;

	UPROPERTY()
	FClientConversationMessage Message;
};


template<>
struct TStructOpsTypeTraits<FConversationTaskResult> : public TStructOpsTypeTraitsBase2<FConversationTaskResult>
{
	enum
	{
		WithNoInitConstructor = true
	};
};

//////////////////////////////////////////////////////////////////////

// Information about a currently active conversation
USTRUCT(BlueprintType)
struct FConversationContext
{
	GENERATED_BODY()

public:
	
	static UE_API FConversationContext CreateServerContext(UConversationInstance* InActiveConversation, const UConversationTaskNode* InTaskBeingConsidered);
	static UE_API FConversationContext CreateClientContext(UConversationParticipantComponent* InParticipantComponent, const UConversationTaskNode* InTaskBeingConsidered);

	UE_API FConversationContext CreateChildContext(const UConversationTaskNode* NewTaskBeingConsidered) const;
	UE_API FConversationContext CreateReturnScopeContext(const FConversationNodeHandle& NewReturnScope) const;

	UE_API UWorld* GetWorld() const;

	UE_API FConversationNodeHandle GetCurrentNodeHandle() const;

	UConversationRegistry& GetConversationRegistry() const { check(ConversationRegistry) return *ConversationRegistry; }

	UConversationInstance* GetActiveConversation() const { ensure(IsServerContext()); return ActiveConversation; }
	UConversationInstance* TryGetActiveConversation() const { return IsServerContext() ? ActiveConversation : nullptr; }

	const UConversationTaskNode* GetTaskBeingConsidered() const { return TaskBeingConsidered; }

	const TArray<FConversationNodeHandle>& GetReturnScopeStack() const { return ReturnScopeStack; }

	UE_API const FConversationParticipantEntry* GetParticipant(const FGameplayTag& ParticipantTag) const;

	UConversationParticipantComponent* GetParticipantComponent(const FGameplayTag& ParticipantTag) const
	{
		if (const FConversationParticipantEntry* Participant = GetParticipant(ParticipantTag))
		{
			return Participant->GetParticipantComponent();
		}

		return nullptr;
	}

	AActor* GetParticipantActor(const FGameplayTag& ParticipantTag) const
	{
		if (const FConversationParticipantEntry* Participant = GetParticipant(ParticipantTag))
		{
			return Participant->Actor;
		}

		return nullptr;
	}

	UE_API FConversationParticipants GetParticipantsCopy() const;

	bool IsServerContext() const { return bServer_PRIVATE; }
	bool IsClientContext() const { return bClient_PRIVATE; }

private:
	UPROPERTY()
	TObjectPtr<UConversationRegistry> ConversationRegistry = nullptr;

	UPROPERTY()
	TObjectPtr<UConversationInstance> ActiveConversation = nullptr;

	UPROPERTY()
	TObjectPtr<UConversationParticipantComponent> ClientParticipant = nullptr;

	UPROPERTY()
	TObjectPtr<const UConversationTaskNode> TaskBeingConsidered = nullptr;

	UPROPERTY()
	TArray<FConversationNodeHandle> ReturnScopeStack;

	UPROPERTY()
	bool bServer_PRIVATE = false;

	UPROPERTY()
	bool bClient_PRIVATE = false;
};

//////////////////////////////////////////////////////////////////////

// Wrapper methods from FConversationContext
UCLASS(MinimalAPI)
class UConversationContextHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// SERVER ONLY
	//-----------------------------------------------------------------------

	// Returns the conversation instance object associated with the conversation context provided, or nullptr if not valid
	UFUNCTION(BlueprintPure, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API UConversationInstance* GetConversationInstance(const FConversationContext& Context);

	// Returns the FConversationNodeHandle of the conversation instance associated with this context, or a handle with an invalid FGuid if not possible
	UFUNCTION(BlueprintPure, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API FConversationNodeHandle GetCurrentConversationNodeHandle(const FConversationContext& Context);

	/**
	 * Registers an actor as part of the conversation, that actor doesn't need to have the UConversationParticipantComponent
	 * it won't be added though.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API void MakeConversationParticipant(const FConversationContext& Context, AActor* ParticipantActor, FGameplayTag ParticipantTag);

	//-----------------------------------------------------------------------

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::AdvanceConversation
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API FConversationTaskResult AdvanceConversation(const FConversationContext& Context);

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::AdvanceConversationWithChoice
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, meta=(AutoCreateRefTerm="Choice"), Category=Conversation)
	static UE_API FConversationTaskResult AdvanceConversationWithChoice(const FConversationContext& Context, const FAdvanceConversationRequest& Choice);

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::PauseConversationAndSendClientChoices
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API FConversationTaskResult PauseConversationAndSendClientChoices(const FConversationContext& Context, const FClientConversationMessage& Message);

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::ReturnToLastClientChoice
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API FConversationTaskResult ReturnToLastClientChoice(const FConversationContext& Context);

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::ReturnToCurrentClientChoice
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API FConversationTaskResult ReturnToCurrentClientChoice(const FConversationContext& Context);

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::ReturnToConversationStart
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category=Conversation)
	static UE_API FConversationTaskResult ReturnToConversationStart(const FConversationContext& Context);

	// Constructs and returns a FConversationTaskResult configured with EConversationTaskResultType::AbortConversation
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = Conversation)
	static UE_API FConversationTaskResult AbortConversation(const FConversationContext& Context);

	/** 
	 * Checks the provided task result against any which would end the conversation e.g. EConversationTaskResultType::Invalid 
	 * or EConversationTaskResultType::AbortConversation 
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = Conversation)
	static UE_API bool CanConversationContinue(const FConversationTaskResult& ConversationTasResult);
	
	// SERVER or CLIENT
	//-----------------------------------------------------------------------

	// Returns the conversation participant component belonging to the participant indicated by 'ParticipantTag', or nullptr if not found
	UFUNCTION(BlueprintPure=false, Category=Conversation)
	static UE_API UConversationParticipantComponent* GetConversationParticipant(const FConversationContext& Context, FGameplayTag ParticipantTag);

	// Returns the conversation participant actor indicated by 'ParticipantTag', or nullptr if not found
	UFUNCTION(BlueprintPure=false, Category=Conversation)
	static UE_API AActor* GetConversationParticipantActor(const FConversationContext& Context, FGameplayTag ParticipantTag);

	// Wrapper to find and return any UConversationParticipantComponent belonging to the provided parameter actor
	UFUNCTION(BlueprintPure=false, Category=Conversation)
	static UE_API UConversationParticipantComponent* FindConversationComponent(AActor* Actor);
};

#undef UE_API
