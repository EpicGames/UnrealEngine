// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "GameplayTagContainer.h"

#include "ConversationDatabase.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

struct FEditedDocumentInfo;

class UConversationGraph;
class FConversationCompiler;
class UEdGraph;
class UConversationNode;
class UConversationRegistry;

/**
 * There may be multiple databases with the same entrypoint tag, this struct holds
 * all of those nodes with the same matching tag name, so that the entry point is
 * effectively randomized when there are multiple copies.
 */
USTRUCT(BlueprintType)
struct FConversationEntryList
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = Conversation)
	FGameplayTag EntryTag;

	UPROPERTY(BlueprintReadOnly, Category = Conversation)
	TArray<FGuid> DestinationList;

	UPROPERTY(BlueprintReadOnly, Category = Conversation)
	FString EntryIdentifier;
};

//////////////////////////////////////////////////////////////////////
//
// This struct represents a logical participant in a conversation.
//
// In an active conversation, logical participants are mapped to actual participants
// (e.g., mapping a logical Player to the current player pawn)
//

USTRUCT(BlueprintType)
struct FCommonDialogueBankParticipant
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = Conversation)
	FText FallbackName;

	/** Identifier represented by the component */
	UPROPERTY(EditAnywhere, Category=Identification, meta=(Categories="Conversation.Participant"))
	FGameplayTag ParticipantName;

	UPROPERTY(EditAnywhere, Category = Identification)
	FLinearColor NodeTint = FLinearColor::White;

	//UPROPERTY()
	//UCommonDialogueSpeakerInfo* SpeakerInfo;
};

//////////////////////////////////////////////////////////////////////
// This is a database of conversation graphs and participants
// It is an asset and never instanced.  The conversation registry is used
// at runtime to actually run a conversation rather than referencing these
// database fragments directly.

UCLASS(MinimalAPI)
class UConversationDatabase : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	UE_API UConversationDatabase(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
	UE_API virtual EDataValidationResult ValidateOutBoundConnections(class FDataValidationContext& Context) const;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	const TMap<FGuid, TObjectPtr<UConversationNode>>& GetFullNodeMap() const { return FullNodeMap; }
#endif

	bool IsNodeReachable(const FGuid& NodeGUID) const { return ReachableNodeMap.Contains(NodeGUID); }

	UE_API FLinearColor GetDebugParticipantColor(FGameplayTag ParticipantID) const;

private: // Compiled Data
	
	// Compiled: Entry points
	UPROPERTY(AssetRegistrySearchable)
	int32 CompilerVersion = INDEX_NONE;

	// Compiled: Reachable nodes
	UPROPERTY(BlueprintReadOnly, Category = Conversation, meta = (AllowPrivateAccess = "true"))
	TMap<FGuid, TObjectPtr<UConversationNode>> ReachableNodeMap;

	// Compiled: Entry points
	UPROPERTY(AssetRegistrySearchable, BlueprintReadOnly, Category = Conversation, meta = (AllowPrivateAccess = "true"))
	TArray<FConversationEntryList> EntryTags;

	// Compiled:
	UPROPERTY(AssetRegistrySearchable, BlueprintReadOnly, Category = Conversation, meta = (AllowPrivateAccess = "true"))
	FGameplayTagContainer ExitTags;
	
	// Compiled: 
	UPROPERTY(AssetRegistrySearchable)
	TArray<FGuid> InternalNodeIds;

	// Compiled: 
	UPROPERTY(AssetRegistrySearchable)
	TArray<FGuid> LinkedToNodeIds;

private:
	// List of participant slots
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Conversation, meta = (AllowPrivateAccess = "true"))
	TArray<FCommonDialogueBankParticipant> Speakers;

private:

#if WITH_EDITORONLY_DATA
	// All nodes
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UConversationNode>> FullNodeMap;

	// 'Source code' graphs (of type UConversationGraph)
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> SourceGraphs;

public:
	UE_API FGuid GetGuidFromNode(const UConversationNode* NodeToFind) const;
	UE_API TObjectPtr<class UEdGraphNode> GetSourceGraphNodeFromGuid(FGuid NodeToFind) const;

	// Info about the graphs we last edited
	UPROPERTY()
	TArray<FEditedDocumentInfo> LastEditedDocuments;
#endif

private:
	friend FConversationCompiler;
	friend UConversationRegistry;
};

#undef UE_API
