// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationNode.h"
#include "GameplayTagContainer.h"
#include "ConversationEntryPointNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

UCLASS(MinimalAPI, meta=(DisplayName="Entry Point"))
class UConversationEntryPointNode : public UConversationNodeWithLinks
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Conversation)
	FGameplayTag EntryTag;

	UFUNCTION()
	UE_API virtual FString GetIdentifier();
};

#undef UE_API
