// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationRequirementNode.h"
#include "ConversationMemory.h"

#include "ConversationTypes.h"
#include "Math/RandomStream.h"
#include "ConversationInstance.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

class UConversationChoiceNode;
class UConversationDatabase;
class UConversationInstance;
class UConversationNodeWithLinks;
class UConversationParticipantComponent;
enum class EConversationNodeBranchOrder : uint8;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAllParticipantsNotifiedOfStartEvent, UConversationInstance*, ConversationInstance);

//////////////////////////////////////////////////////////////////////

/**
 * An active conversation between one or more participants
 */
UCLASS(MinimalAPI)
class UConversationInstance : public UObject
{
	GENERATED_BODY()

public: 

	// Server notification sent after all participants have been individually notified of conversation start
	DECLARE_EVENT_OneParam(UConversationInstance, FOnAllParticipantsNotifiedOfStartEvent, UConversationInstance* /*ConversationInstance*/);
	FOnAllParticipantsNotifiedOfStartEvent OnAllParticipantsNotifiedOfStart;

public:
	UE_API UConversationInstance();

	UE_API virtual UWorld* GetWorld() const override;
	
#if WITH_SERVER_CODE
	/** Should be called with a copy of the conversation participants before any removals happen, that way clients can properly respond to the end of their respective conversations
	  * with an accurate account of who was in that conversation. */
	UE_API void ServerRemoveParticipant(const FGameplayTag& ParticipantID, const FConversationParticipants& PreservedParticipants);

	UE_API void ServerAssignParticipant(const FGameplayTag& ParticipantID, AActor* ParticipantActor);

	UE_API void ServerStartConversation(const FGameplayTag& EntryPoint, const UConversationDatabase* Graph = nullptr, const FString& EntryPointIdentifier = FString());

	UE_API void ServerAdvanceConversation(const FAdvanceConversationRequest& InChoicePicked);

	UE_API virtual void OnInvalidBranchChoice(const FAdvanceConversationRequest& InChoicePicked);

	UE_API void ServerAbortConversation();

	UE_API void ServerRefreshConversationChoices();

	UE_API void ServerRefreshTaskChoiceData(const FConversationNodeHandle& Handle);

	/** Attempts to process the current conversation node again - only useful in very specific circumstances where you'd want to re-run the current node
	  * without having to deal with conversation flow changes. */
	UE_API void ServerRefreshCurrentConversationNode();

	/**
     * This is memory that will last for the duration of the conversation instance.  Don't store
     * anything here you want to be long lived.
	 */
	FConversationMemory& GetInstanceMemory() { return InstanceMemory; }

	UE_API TArray<FGuid> DetermineBranches(const TArray<FGuid>& SourceList, EConversationRequirementResult MaximumRequirementResult = EConversationRequirementResult::Passed);
#endif

	//@TODO: Conversation: Meh
	TArray<FConversationParticipantEntry> GetParticipantListCopy() const
	{
		return Participants.List;
	}

	const TArray<FConversationParticipantEntry>& GetParticipantList() const
	{
		return Participants.List;
	}

	FConversationParticipants GetParticipantsCopy() const
	{
		return Participants;
	}

	const FConversationParticipantEntry* GetParticipant(FGameplayTag ParticipantID) const
	{
		return Participants.GetParticipant(ParticipantID);
	}

	UConversationParticipantComponent* GetParticipantComponent(FGameplayTag ParticipantID) const
	{
		return Participants.GetParticipantComponent(ParticipantID);
	}

	const UConversationDatabase* GetActiveConversationGraph() const
	{
		return ActiveConversationGraph.Get();
	}

	const FConversationNodeHandle& GetCurrentNodeHandle() const { return CurrentBranchPoint.GetNodeHandle(); }
	const FConversationChoiceReference& GetCurrentChoiceReference() const { return CurrentBranchPoint.ClientChoice.ChoiceReference; }
	const TArray<FClientConversationOptionEntry>& GetCurrentUserConversationChoices() const { return CurrentUserChoices; }

protected:
	virtual void OnStarted() { }
	virtual void OnEnded() { }

#if WITH_SERVER_CODE
	UE_API void ModifyCurrentConversationNode(const FConversationChoiceReference& NewChoice);
	UE_API void ModifyCurrentConversationNode(const FConversationBranchPoint& NewBranchPoint);
	UE_API void ReturnToLastClientChoice(const FConversationContext& Context);
	UE_API void ReturnToCurrentClientChoice(const FConversationContext& Context);
	UE_API void ReturnToStart(const FConversationContext& Context);
	UE_API virtual void PauseConversationAndSendClientChoices(const FConversationContext& Context, const FClientConversationMessage& ClientMessage);
	virtual void OnChoiceNodePickedByUser(const FConversationContext& Context, const UConversationChoiceNode* ChoiceNode, const TArray<FConversationBranchPoint>& ValidDestinations) {};
#endif

private:
	UE_API bool AreAllParticipantsReadyToConverse() const;
	UE_API void TryStartingConversation();

	const FConversationBranchPoint& GetCurrentBranchPoint() const { return CurrentBranchPoint; }

	UE_API void ResetConversationProgress();
	UE_API void UpdateNextChoices(const FConversationContext& Context);
	UE_API void SetNextChoices(const TArray<FConversationBranchPoint>& InAllChoices);
	UE_API const FConversationBranchPoint* FindBranchPointFromClientChoice(const FConversationChoiceReference& InChoice) const;

#if WITH_SERVER_CODE
	UE_API void OnCurrentConversationNodeModified();

	UE_API void ProcessCurrentConversationNode();

	// Chooses the next branch index. Called when the conversation advanced and the user did not select a choice
	UE_API int32 ChooseBranchIndex(EConversationNodeBranchOrder BranchOrder, int32 NumBranchIndices);
#endif //WITH_SERVER_CODE

protected:
	const FGameplayTagContainer& GetCurrentChoiceTags() const { return CurrentBranchPoint.ClientChoice.ChoiceTags; }

	TArray<FClientConversationOptionEntry> CurrentUserChoices;

private:
	UPROPERTY()
	FConversationParticipants Participants;

	UPROPERTY()
	TObjectPtr<const UConversationDatabase> ActiveConversationGraph = nullptr;

	FGameplayTag StartingEntryGameplayTag;
	FConversationBranchPoint StartingBranchPoint;

	FConversationBranchPoint CurrentBranchPoint;

	struct FCheckpoint
	{
		FConversationBranchPoint ClientBranchPoint;
		TArray<FConversationChoiceReference> ScopeStack;
	};

	TArray<FCheckpoint> ClientBranchPoints;

	TArray<FConversationBranchPoint> CurrentBranchPoints;

	TArray<FConversationChoiceReference> ScopeStack;

	FRandomStream ConversationRNG;

private:
#if WITH_SERVER_CODE
	FConversationMemory InstanceMemory;
#endif

private:
	bool bConversationStarted = false;

public:
	const bool HasConversationStarted() const { return bConversationStarted; }
};

#undef UE_API
