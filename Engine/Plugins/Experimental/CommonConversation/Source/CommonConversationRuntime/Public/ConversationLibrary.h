// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Kismet/BlueprintFunctionLibrary.h"

#include "Templates/SubclassOf.h"
#include "ConversationLibrary.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

struct FGameplayTag;

class AActor;
class UConversationDatabase;
class UConversationInstance;

UCLASS(MinimalAPI)
class UConversationLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	UE_API UConversationLibrary();

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="Conversation", meta = (DeterminesOutputType = "ConversationType"))
	static UE_API UConversationInstance* StartConversation(const FGameplayTag& ConversationEntryTag, AActor* Instigator, const FGameplayTag& InstigatorTag,
		AActor* Target, const FGameplayTag& TargetTag, const TSubclassOf<UConversationInstance> ConversationInstanceClass = {});

	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category = "Conversation", meta = (DeterminesOutputType = "ConversationType"))
	static UE_API UConversationInstance* StartConversationFromGraph(const FGameplayTag& ConversationEntryTag, AActor* Instigator, const FGameplayTag& InstigatorTag,
		AActor* Target, const FGameplayTag& TargetTag, const UConversationDatabase* Graph, const FString& EntryPointIdentifier = TEXT(""));
};

#undef UE_API
