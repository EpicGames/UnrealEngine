// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationTaskNode.h"

#include "GameplayTagContainer.h"
#include "ConversationLinkNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

UCLASS(MinimalAPI)
class UConversationLinkNode : public UConversationTaskNode
{
	GENERATED_BODY()

public:
	UE_API UConversationLinkNode();

	FGameplayTag GetRemoteEntryTag() const { return RemoteEntryTag; }

protected:
	UE_API virtual FConversationTaskResult ExecuteTaskNode_Implementation(const FConversationContext& Context) const override;
	UE_API virtual void GatherChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = "Link")
	FGameplayTag RemoteEntryTag;
};

#undef UE_API
