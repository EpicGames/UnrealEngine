// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

class UConversationTaskNode;

/**
 * Used to store arbitrary structs of data in different contexts for a conversation.  During a conversation
 * an NPC may need to remember a choice or remember a randomly chosen thing.  This memory store allows that
 * across different contexts, such as for the duration of the conversation instance, or as memory on the NPC.
 */
class FConversationMemory : public FNoncopyable
{
public:
	FConversationMemory() { }
	UE_API ~FConversationMemory();
	
	/** NOTE: It is not valid for your memory struct to store a RAW UObject*, this data is not scanned for garbage collection. */
	UE_API void* GetTaskMemoryOfType(const UConversationTaskNode& Task, const UScriptStruct* TaskMemoryStructType);

	/** NOTE: It is not valid for your memory struct to store a RAW UObject*, this data is not scanned for garbage collection. */
	template <typename TStructTaskMemory>
	TStructTaskMemory* GetTaskMemory(const UConversationTaskNode& Task)
	{
		return static_cast<TStructTaskMemory*>(GetTaskMemoryOfType(Task, TStructTaskMemory::StaticStruct()));
	}
	
private:
	struct FConversationTaskMemoryKey
	{
	public:
		FConversationTaskMemoryKey(const UConversationTaskNode& SourceTaskInstance, const UScriptStruct* TaskMemoryStructClass);

		const UScriptStruct* GetTaskMemoryStruct() const;

		bool operator==(const FConversationTaskMemoryKey& Rhs) const { return SourceTaskInstanceKey == Rhs.SourceTaskInstanceKey && TaskMemoryStructClassKey == Rhs.TaskMemoryStructClassKey; }
		friend inline uint32 GetTypeHash(const FConversationTaskMemoryKey& Key) { return Key.KeyHash; }

	private:
		FObjectKey SourceTaskInstanceKey;
		FObjectKey TaskMemoryStructClassKey;
		uint32 KeyHash;
	};

	TMap<FConversationTaskMemoryKey, void*> TaskMemory;

	//@TODO: Conversation: Copied from the SlabAllocator, we should have something like this in Core.
	class FTaskMemoryAllocator
	{
	public:
		FTaskMemoryAllocator()
			: SlabSize(16384)
		{
		}

		~FTaskMemoryAllocator()
		{
			for (void* Slab : Slabs)
			{
				FMemory::Free(Slab);
			}
		}

		void* Allocate(uint64 Size)
		{
			uint64 AllocationSize = Align(Size, 16);
			if (!CurrentSlab || CurrentSlabAllocatedSize + AllocationSize > SlabSize)
			{
				TotalAllocatedSize += SlabSize;
				void* Allocation = FMemory::Malloc(SlabSize, 16);
				CurrentSlab = reinterpret_cast<uint8*>(Allocation);
				CurrentSlabAllocatedSize = 0;
				Slabs.Add(CurrentSlab);
			}
			void* Allocation = CurrentSlab + CurrentSlabAllocatedSize;
			CurrentSlabAllocatedSize += AllocationSize;
			return Allocation;
		}

	private:
		TArray<void*> Slabs;
		uint8* CurrentSlab = nullptr;
		const uint64 SlabSize;
		uint64 CurrentSlabAllocatedSize = 0;
		uint64 TotalAllocatedSize = 0;
	};

	FTaskMemoryAllocator TaskMemoryAllocator;
};

#undef UE_API
