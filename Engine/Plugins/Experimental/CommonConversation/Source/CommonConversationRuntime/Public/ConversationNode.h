// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ConversationNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

struct FGameplayTag;

class UWorld;

class UConversationNode;
class UConversationDatabase;
class UCommonDialogueConversation;
struct FConversationContext;

/**
 * 
 */
enum class EConversationNodeDescriptionVerbosity : uint8 
{
	Basic,
	Detailed,
};

//////////////////////////////////////////////////////////////////////

/**
 * 
 */
USTRUCT(BlueprintType)
struct FConversationNodeHandle
{
	GENERATED_BODY()

public:
	FConversationNodeHandle() : NodeGUID(0, 0, 0, 0) {}
	FConversationNodeHandle(const FGuid& InNodeGUID) : NodeGUID(InNodeGUID) { }

	UPROPERTY(EditAnywhere, Category=Conversation)
	FGuid NodeGUID;

	bool IsValid() const { return NodeGUID.IsValid(); }
	FString ToString() const { return NodeGUID.ToString(); }
	void Invalidate() { NodeGUID.Invalidate(); }

	FConversationNodeHandle& operator = (const FGuid& InNodeGUID)
	{
		NodeGUID = InNodeGUID;
		return *this;
	}

	operator FGuid& () { return NodeGUID; }
	operator FGuid() const { return NodeGUID; }

	/** 
	 * Tries to resolve the node, this may fail, the guid might be bogus, or the node might not be
	 * in memory.
	 */
	UE_API const UConversationNode* TryToResolve(const FConversationContext& Context) const;
	UE_API const UConversationNode* TryToResolve_Slow(UWorld* InWorld, const UConversationDatabase* Graph = nullptr) const;
};

inline bool operator==(const FConversationNodeHandle& Lhs, const FConversationNodeHandle& Rhs) { return Lhs.NodeGUID == Rhs.NodeGUID; }
inline bool operator!=(const FConversationNodeHandle& Lhs, const FConversationNodeHandle& Rhs) { return !(Lhs == Rhs); }

//////////////////////////////////////////////////////////////////////

// Represents a single runtime node in the conversation database.
UCLASS(MinimalAPI, Abstract, Const)
class UConversationNode : public UObject
{
	GENERATED_UCLASS_BODY()

	UE_API virtual UWorld* GetWorld() const override;

	/** fill in data about tree structure */
	UE_API void InitializeNode(UConversationNode* InParentNode);

	/** initialize any asset related data */
	UE_API virtual void InitializeFromAsset(UConversationDatabase& Asset);

	/** gathers description of all runtime parameters */
	UE_API virtual void DescribeRuntimeValues(const UCommonDialogueConversation& OwnerComp, EConversationNodeDescriptionVerbosity Verbosity, TArray<FString>& Values) const;

	/** @return parent node */
	UConversationNode* GetParentNode() const { return ParentNode; }

	/** @return name of node */
	UE_API FText GetDisplayNameText() const;

	/** @return string containing description of this node instance with all relevant runtime values */
	UE_API FText GetRuntimeDescription(const UCommonDialogueConversation& OwnerComp, EConversationNodeDescriptionVerbosity Verbosity) const;

	/** @return string containing description of this node with all setup values */
	UE_API virtual FText GetStaticDescription() const;

	/** The node's unique ID. */
	FGuid GetNodeGuid() const { return Compiled_NodeGUID; }

#if WITH_EDITOR
	/** Get the name of the icon used to display this node in the editor */
	UE_API virtual FName GetNodeIconName() const;

	/** Called after creating new node in behavior tree editor, use for versioning */
	virtual void OnNodeCreated() {}

	bool ShowPropertyEditors() const
	{
		return bShowPropertyEditors;
	}

	bool ShowPropertyDetails() const
	{
		if (bShowPropertyEditors)
		{
			return false;
		}

		return bShowPropertyDetails;
	}
#endif

protected:
	UFUNCTION(BlueprintCallable, Category=Conversation)
	UE_API FLinearColor GetDebugParticipantColor(FGameplayTag ParticipantID) const;

#if WITH_EDITOR
	/** @return true if this property should be hidden in static description, false if should be shown */
	UE_API bool ShouldHideProperty(FProperty* InTestProperty) const;
#endif // #if WITH_EDITOR

public:
	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UObject> EvalWorldContextObj;

protected:
	/** node name */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Description, AdvancedDisplay, meta = (AllowPrivateAccess = "true"))
	FString NodeName;

	/**
	 * The node's unique ID.  This value is set during compilation.
	 */
	UPROPERTY(BlueprintReadOnly, Category = Conversation, meta = (AllowPrivateAccess = "true"))
	FGuid Compiled_NodeGUID;

#if WITH_EDITORONLY_DATA
	/** show detailed information about properties */
	UPROPERTY(EditInstanceOnly, Category=Description, AdvancedDisplay)
	uint32 bShowPropertyDetails : 1;

	/** show detailed information about properties */
	UPROPERTY(EditInstanceOnly, Category=Description, AdvancedDisplay)
	uint32 bShowPropertyEditors : 1;
#endif

private:
	/** parent node */
	UPROPERTY()
	TObjectPtr<UConversationNode> ParentNode;

private:
	friend class FConversationCompiler;
};

//////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI, Abstract)
class UConversationNodeWithLinks : public UConversationNode
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = Conversation)
	TArray<FGuid> OutputConnections;

	virtual bool IsOutBoundConnectionAllowed(const UConversationNodeWithLinks* OtherNode, FText& OutErrorMessage) const { return true; }

	friend class UConversationRegistry;
};

#undef UE_API
