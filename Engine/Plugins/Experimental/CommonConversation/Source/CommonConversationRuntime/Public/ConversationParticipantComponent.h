// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationMemory.h"
#include "ConversationTypes.h"

#include "ConversationParticipantComponent.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

class UConversationInstance;
struct FConversationNodeHandle;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnConversationStatusChanged, bool, bIsInConversation);

/**
 * Active conversation participants should have this component on them.
 * It keeps track of what conversations they are participating in (typically no more than one)
 */
UCLASS(MinimalAPI, BlueprintType)
class UConversationParticipantComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API UConversationParticipantComponent();

	// Client and server notification of the conversation starting or ending
	DECLARE_EVENT_OneParam(UConversationParticipantComponent, FConversationStatusChangedEvent, bool /*Started*/);
	FConversationStatusChangedEvent ConversationStatusChanged;

	DECLARE_EVENT(UConversationParticipantComponent, FConversationStartedEvent);
	FConversationStartedEvent ConversationStarted;

	DECLARE_EVENT_OneParam(UConversationParticipantComponent, FConversationUpdatedEvent, const FClientConversationMessagePayload& /*Message*/);
	FConversationUpdatedEvent ConversationUpdated;

	DECLARE_EVENT_TwoParams(UConversationParticipantComponent, FConversationTaskChoiceDataUpdatedEvent, const FConversationNodeHandle& /*Handle*/, const FClientConversationOptionEntry& /*OptionEntry*/);
	FConversationTaskChoiceDataUpdatedEvent ConversationTaskChoiceDataUpdated;

public:
	UE_API void SendClientConversationMessage(const FConversationContext& Context, const FClientConversationMessagePayload& Payload);
	UE_API virtual void SendClientUpdatedChoices(const FConversationContext& Context, const bool bForcedRefresh = false);
	UE_API void SendClientRefreshedTaskChoiceData(const FConversationNodeHandle& Handle, const FConversationContext& Context);

	UFUNCTION(BlueprintCallable, Category=Conversation)
	UE_API void RequestServerAdvanceConversation(const FAdvanceConversationRequest& InChoicePicked);

#if WITH_SERVER_CODE
	UE_API void ServerNotifyConversationStarted(UConversationInstance* Conversation, FGameplayTag AsParticipant);
	UE_API void ServerNotifyConversationEnded(UConversationInstance* Conversation, const FConversationParticipants& PreservedParticipants);
	UE_API void ServerNotifyExecuteTaskAndSideEffects(const FConversationNodeHandle& Handle, const UConversationDatabase* Graph = nullptr);
	UE_API void ServerForAllConversationsRefreshChoices(UConversationInstance* IgnoreConversation = nullptr);
	UE_API void ServerForAllConversationsRefreshTaskChoiceData(const FConversationNodeHandle& Handle, UConversationInstance* IgnoreConversation /*= nullptr*/);

	/** Check if this actor is in a good state to start a conversation */
	virtual bool ServerIsReadyToConverse() const { return true; }

	/** Ask to this actor to change is state to be able to start a conversation */
	UE_API virtual void ServerGetReadyToConverse();

	/** Ask this actor to abort all active conversations */
	UE_API void ServerAbortAllConversations();

	FConversationMemory& GetParticipantMemory() { return ParticipantMemory; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FParticipantReadyToConverse, UConversationParticipantComponent*);
	/** Delegate send when this actor enter in the good state to start a conversation */
	FParticipantReadyToConverse OnParticipantReadyToConverseEvent;
#endif

	UFUNCTION(BlueprintCallable, Category=Conversation)
	UE_API virtual FText GetParticipantDisplayName();

	UFUNCTION(BlueprintCallable, Category = Conversation)
	UE_API bool IsInActiveConversation() const;

public:

	UE_API FConversationNodeHandle GetCurrentNodeHandle() const;

	UE_API const FConversationParticipantEntry* GetParticipant(const FGameplayTag& ParticipantTag) const;

	UFUNCTION(BlueprintPure, Category = Conversation)
	UE_API AActor* GetParticipantActor(const FGameplayTag& ParticipantTag) const;

	UFUNCTION(BlueprintCallable, Category = Conversation)
	UE_API void GetOtherParticipantActors(TArray<AActor*>& OutParticipantActors) const;

protected:
	UFUNCTION(Server, Reliable)
	UE_API void ServerAdvanceConversation(const FAdvanceConversationRequest& InChoicePicked);

	UFUNCTION(Client, Reliable)
	UE_API void ClientUpdateParticipants(const FConversationParticipants& InParticipants);

	UFUNCTION(Client, Reliable)
	UE_API void ClientExecuteTaskAndSideEffects(FConversationNodeHandle Handle, const UConversationDatabase* Graph = nullptr);

	UFUNCTION(Client, Reliable)
	UE_API void ClientUpdateConversation(const FClientConversationMessagePayload& Message);

	UFUNCTION(Client, Reliable)
	UE_API void ClientUpdateConversationTaskChoiceData(FConversationNodeHandle Handle, const FClientConversationOptionEntry& OptionEntry);

	UFUNCTION(Client, Reliable)
	UE_API void ClientUpdateConversations(int32 InConversationsActive);

	UFUNCTION(Client, Reliable)
	UE_API void ClientStartConversation(const FConversationParticipants& InParticipants);

	UFUNCTION(Client, Reliable)
	UE_API void ClientExitConversation(const FConversationParticipants& InParticipants);

protected:
	UFUNCTION()
	UE_API void OnRep_ConversationsActive(int32 OldConversationsActive);

	UE_API virtual void OnEnterConversationState();
	UE_API virtual void OnLeaveConversationState();
	UE_API virtual void OnConversationUpdated(const FClientConversationMessagePayload& Message);

	UE_API virtual void OnClientStartConversation(const FConversationParticipants& InParticipants);
	UE_API virtual void OnClientExitConversation(const FConversationParticipants& InParticipants);

#if WITH_SERVER_CODE
	UE_API virtual void OnServerConversationStarted(UConversationInstance* Conversation, FGameplayTag AsParticipant);
	UE_API virtual void OnServerConversationEnded(UConversationInstance* Conversation);
#endif

#if WITH_SERVER_CODE
	UConversationInstance* GetCurrentConversationForAuthority() const { return Auth_CurrentConversation; }
	const TArray<UConversationInstance*>& GetConversationsForAuthority() const { return Auth_Conversations; }
#endif

public:
    // The number of conversations active.  A given conversationalist might be in multiple conversations at once.
    // e.g. Multiple players "talking" to the same NPC in a multiplayer game.
	int32 GetConversationsActive() const { return ConversationsActive; }
	
	// A cached version of the last conversation message payload data recieved.
	const FClientConversationMessagePayload& GetLastMessage() const { return LastMessage; }
	
	// Gets the last message index recieved, this is just a monotomically increasing number each time we get a new message.
	const int32 GetLastMessageIndex() const { return MessageIndex; }

	bool GetIsFirstConversationUpdateBroadcasted() const { return bIsFirstConversationUpdateBroadcasted; }

private:

	UPROPERTY(Replicated, ReplicatedUsing=OnRep_ConversationsActive)
	int32 ConversationsActive = 0;

private:
#if WITH_SERVER_CODE
	FConversationMemory ParticipantMemory;
#endif

private:
	UPROPERTY()
	TObjectPtr<UConversationInstance> Auth_CurrentConversation;

	UPROPERTY()
	TArray<TObjectPtr<UConversationInstance>> Auth_Conversations;
	
	UPROPERTY()
    FClientConversationMessagePayload LastMessage;
    
    int32 MessageIndex = 0;
    
	bool bIsFirstConversationUpdateBroadcasted = false;
};

//////////////////////////////////////////////////////////////////////

#undef UE_API
