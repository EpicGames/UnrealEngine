// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureStateChangeObserver.h"
#include "GameplayTagContainer.h"
#include "Subsystems/WorldSubsystem.h"

#include "ConversationRegistry.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

class UConversationNode;
class UConversationRegistry;

class UGameFeatureData;
class UConversationDatabase;
class UWorld;
struct FStreamableHandle;

//  Container for safely replicating  script struct references (constrained to a specified parent struct)
USTRUCT()
struct FNetSerializeScriptStructCache_ConvVersion
{
	GENERATED_BODY()

	UE_API void InitForType(UScriptStruct* InScriptStruct);

	// Serializes reference to given script struct (must be in the cache)
	UE_API bool NetSerialize(FArchive& Ar, UScriptStruct*& Struct);

	UPROPERTY()
	TMap<TObjectPtr<UScriptStruct>, int32> ScriptStructsToIndex;

	UPROPERTY()
	TArray<TObjectPtr<UScriptStruct>> IndexToScriptStructs;
};

/**
 * These handles are issued when someone requests a conversation entry point be streamed in.
 * As long as this handle remains active, we were continue to keep it keep those elements streamed
 * in, as well as if new game feature plugins activate, we will stream in additional assets
 * or let previous ones expire.
 */
struct FConversationsHandle : public TSharedFromThis<FConversationsHandle>
{
	static UE_API TSharedPtr<FConversationsHandle> Create(const UConversationRegistry* InOwningRegistry, const TSharedPtr<FStreamableHandle>& InStreamableHandle, const TArray<FGameplayTag>& InEntryTags);

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	UE_API FConversationsHandle(FPrivateToken, UConversationRegistry* InOwningRegistry, const TSharedPtr<FStreamableHandle>& InStreamableHandle, const TArray<FGameplayTag>& InEntryTags);

private:
	void Initialize();
	void HandleAvailableConversationsChanged();

private:
	TSharedPtr<FStreamableHandle> StreamableHandle;
	TArray<FGameplayTag> ConversationEntryTags;
	TWeakObjectPtr<UConversationRegistry> OwningRegistryPtr;
};


DECLARE_MULTICAST_DELEGATE(FAvailableConversationsChangedEvent);

/**
 * A registry that can answer questions about all available dialogue assets
 */
UCLASS(MinimalAPI)
class UConversationRegistry : public UWorldSubsystem, public IGameFeatureStateChangeObserver
{
	GENERATED_BODY()

public:
	UE_API UConversationRegistry();

	static UE_API UConversationRegistry* GetFromWorld(const UWorld* World);

	/** UWorldSubsystem Begin */
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	/** UWorldSubsystem End */

	UE_API UConversationNode* GetRuntimeNodeFromGUID(const FGuid& NodeGUID, const UConversationDatabase* Graph = nullptr) const;
	UE_API UConversationNode* TryGetRuntimeNodeFromGUID(const FGuid& NodeGUID, const UConversationDatabase* Graph = nullptr) const;
	UE_API TArray<FGuid> GetEntryPointGUIDs(const FGameplayTag& EntryPoint) const;

	UE_API TArray<FGuid> GetOutputLinkGUIDs(const FGameplayTag& EntryPoint) const;
	UE_API TArray<FGuid> GetOutputLinkGUIDs(const FGuid& SourceGUID) const;
	UE_API TArray<FGuid> GetOutputLinkGUIDs(const TArray<FGuid>& SourceGUIDs) const;
	UE_API TArray<FGuid> GetOutputLinkGUIDs(const UConversationDatabase* Graph, const FGameplayTag& EntryPoint, const FString& EntryIdentifier) const;
	UE_API TArray<FGuid> GetOutputLinkGUIDs(const UConversationDatabase* Graph, const FGuid& SourceGUID) const;

	UE_API TSharedPtr<FConversationsHandle> LoadConversationsFor(const FGameplayTag& ConversationEntryTag) const;
	UE_API TSharedPtr<FConversationsHandle> LoadConversationsFor(const TArray<FGameplayTag>& ConversationEntryTags) const;

	UE_API TArray<FPrimaryAssetId> GetPrimaryAssetIdsForEntryPoint(FGameplayTag EntryPoint) const;

	// If a conversation database links to other conversaton assets, the tags of those conversations can be obtained here
	UE_API TArray<FGameplayTag> GetLinkedExitConversationEntryTags(const UConversationDatabase* ConversationDatabase) const;

	UPROPERTY(Transient)
	FNetSerializeScriptStructCache_ConvVersion ConversationChoiceDataStructCache;

	FAvailableConversationsChangedEvent AvailableConversationsChanged;

	UE_API void FlushBuildDependenciesGraph();
	UE_API void SetWaitDependenciesGraphToFlush(bool bWaitToFlush);
	
private:
	UConversationDatabase* GetConversationFromNodeGUID(const FGuid& NodeGUID) const;

	void BuildDependenciesGraph();
	void GetAllDependenciesForConversation(const FSoftObjectPath& Parent, TSet<FSoftObjectPath>& OutConversationsToLoad) const;

	void GameFeatureStateModified();

	UE_API virtual void OnGameFeatureActivated(const UGameFeatureData* GameFeatureData, const FString& PluginURL) override;

	UE_API virtual void OnGameFeatureDeactivating(const UGameFeatureData* GameFeatureData, FGameFeatureDeactivatingContext& Context, const FString& PluginURL) override;

private:
	TMap<FSoftObjectPath, TArray<FSoftObjectPath>> RuntimeDependencyGraph;
	TMap<FGameplayTag, TArray<FSoftObjectPath>> EntryTagToConversations;
	TMap<FGameplayTag, TArray<FGuid>> EntryTagToEntryList;
	TMap<FGuid, FSoftObjectPath> NodeGuidToConversation;

private:
	bool bDependenciesBuilt = false;
	bool bWaitForDependenciesGraphFlush = false;
};

#undef UE_API
