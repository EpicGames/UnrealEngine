// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationSubNode.h"
#include "ConversationRequirementNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

/**
 * The requirement result.
 */
UENUM(BlueprintType)
enum class EConversationRequirementResult : uint8
{
	/** This option is available */
	Passed,
	/** This option is not available, but we should tell the player about it still. */
	FailedButVisible,
	/** This option is not available, and we should keep it hidden. */
	FailedAndHidden,
};

COMMONCONVERSATIONRUNTIME_API EConversationRequirementResult MergeRequirements(EConversationRequirementResult CurrentResult, EConversationRequirementResult MergeResult);

/**
 *  A requirement is placed on a parent node to control whether or not it can be activated
 *  (when a link to the parent node is being evaluated, the requirement will be asked if it is satisfied or not)
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UConversationRequirementNode : public UConversationSubNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent)
	UE_API EConversationRequirementResult IsRequirementSatisfied(const FConversationContext& Context) const;
};

#undef UE_API
