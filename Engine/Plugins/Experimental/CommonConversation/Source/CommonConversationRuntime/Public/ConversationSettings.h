// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "ConversationInstance.h"

#include "ConversationSettings.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

/**
 * Conversation settings.
 */
UCLASS(MinimalAPI, Config = Game, DefaultConfig, meta = (DisplayName = "Conversation"))
class UConversationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UE_API UConversationSettings();

	UClass* GetConversationInstanceClass() const { return ConversationInstanceClass.LoadSynchronous(); }

protected:

	UPROPERTY(config, EditAnywhere, Category=Conversation)
	TSoftClassPtr<UConversationInstance> ConversationInstanceClass;
};

#undef UE_API
