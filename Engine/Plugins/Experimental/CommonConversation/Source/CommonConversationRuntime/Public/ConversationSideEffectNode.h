// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationSubNode.h"
#include "ConversationSideEffectNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

struct FConversationContext;

/**
 * Side effects are actions that are performed just after a task is executed
 * (this allows state-altering or cosmetic actions to be mixed in to other nodes)
 * 
 * When a task executes on the server, it replicates to the client that it executed and
 * to then execute any client side effects that may be necessary for that task.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UConversationSideEffectNode : public UConversationSubNode
{
	GENERATED_BODY()

public:
	/** Called by the client and server code executes the side effect. */
	UE_API void CauseSideEffect(const FConversationContext& Context) const;

protected:
	UFUNCTION(BlueprintNativeEvent, BlueprintAuthorityOnly)
	UE_API void ServerCauseSideEffect(const FConversationContext& Context) const;

	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic)
	UE_API void ClientCauseSideEffect(const FConversationContext& Context) const;
};

#undef UE_API
