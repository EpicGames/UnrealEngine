// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversationNode.h"

#include "ConversationTaskNode.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

enum class EConversationRequirementResult : uint8;
struct FConversationBranchPointBuilder;
struct FConversationNodeParameterPair;
struct FConversationTaskResult;

class UConversationSubNode;

UENUM()
enum class EConversationNodeBranchOrder : uint8
{
	Random,
	LeftToRight,
	RightToLeft
};

/** 
 * Task are leaf nodes of behavior tree, which perform actual actions
 *
 * Because some of them can be instanced for specific AI, following virtual functions are not marked as const:
 *  - ExecuteTask
 *  - AbortTask
 *  - TickTask
 *  - OnMessage
 *
 * If your node is not being instanced (default behavior), DO NOT change any properties of object within those functions!
 * Template nodes are shared across all behavior tree components using the same tree asset and must store
 * their runtime properties in provided NodeMemory block (allocation size determined by GetInstanceMemorySize() )
 *
 */

/**
 * The ConversationTaskNode is the basis of any task in the conversation graph,
 * that task may be as simple as saying some text to the user, and providing some choices.
 * However more complex tasks can fire off quests, can spawn actors, pretty much any arbitrary
 * thing you want.
 * 
 * The conversation system is less about just a dialogue tree, and more about a graph of
 * actions the NPC can take, and choices they can provide to the player.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UConversationTaskNode : public UConversationNodeWithLinks
{
	GENERATED_BODY()

public:
	// Requirements and side effects
	UPROPERTY(BlueprintReadOnly, Category = Conversation)
	TArray<TObjectPtr<UConversationSubNode>> SubNodes;

#if WITH_EDITORONLY_DATA
	/* EDITOR ONLY VISUALS: Does this task internally have requirements? */
	UPROPERTY(EditDefaultsOnly, Category=Conversation)
	uint32 bHasRequirements : 1;

	/** EDITOR ONLY VISUALS: Does this task generate dynamic choices? */
	UPROPERTY(EditDefaultsOnly, Category=Conversation)
	uint32 bHasDynamicChoices : 1;
#endif

	//This setting is designed for requirements that only matter when option is generated
	UPROPERTY(EditDefaultsOnly, Category = "UI")
	bool bIgnoreRequirementsWhileAdvancingConversations = false;

	/**
	 * Defines the order in which branches are selected when the conversation advanced automatically and the user did not choose a specific branch
	 */
	UPROPERTY(EditInstanceOnly, Category=Conversation, AdvancedDisplay)
	EConversationNodeBranchOrder BranchOrder = EConversationNodeBranchOrder::Random;

public:
	UFUNCTION(BlueprintNativeEvent)
	UE_API bool GetNodeBodyColor(FLinearColor& BodyColor) const;

public:
	//@TODO: CONVERSATION: Comment me
	UE_API FConversationTaskResult ExecuteTaskNodeWithSideEffects(const FConversationContext& Context) const;

public:
#if WITH_EDITOR
	UE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

	/**
	 * Returns the highest priority EConversationRequirementResult, so Passed, having the least priority
	 * and FailedHidden having the highest priority.
	 */
	UE_API EConversationRequirementResult CheckRequirements(const FConversationContext& InContext) const;

	static UE_API void GenerateChoicesForDestinations(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& InContext, const TArray<FGuid>& CandidateDestinations);

	/** Generates this task nodes' next possible conversation chioces. Default behavior handles finding static destination node choices and dynamic choices
	  * generated by the node, but this can be overriden if more custom behavior is desired for a specific task node. */
	UE_API virtual void GenerateNextChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const;

protected:
	UFUNCTION(BlueprintNativeEvent, BlueprintAuthorityOnly)
	UE_API EConversationRequirementResult IsRequirementSatisfied(const FConversationContext& Context) const;

	UFUNCTION(BlueprintNativeEvent, BlueprintAuthorityOnly)
	UE_API FConversationTaskResult ExecuteTaskNode(const FConversationContext& Context) const;

	UFUNCTION(BlueprintNativeEvent, BlueprintCosmetic)
	UE_API void ExecuteClientEffects(const FConversationContext& Context) const;

	UFUNCTION(BlueprintNativeEvent, BlueprintAuthorityOnly)
	UE_API void GatherStaticExtraData(const FConversationContext& Context, TArray<FConversationNodeParameterPair>& InOutExtraData) const;

	UE_API virtual void GatherChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const;

	UE_API virtual void GatherStaticChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const;
	UE_API virtual void GatherDynamicChoices(FConversationBranchPointBuilder& BranchBuilder, const FConversationContext& Context) const;

protected:
#if WITH_EDITORONLY_DATA
	/** Default color of the node. */
	UPROPERTY(EditDefaultsOnly, Category=Description)
	FLinearColor DefaultNodeBodyColor = FLinearColor::White;
#endif
};

#undef UE_API
