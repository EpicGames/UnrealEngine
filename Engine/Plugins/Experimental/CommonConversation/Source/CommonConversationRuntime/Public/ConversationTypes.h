// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "GameFramework/Actor.h" // IWYU pragma: keep
#include "ConversationNode.h"
#include "ConversationTypes.generated.h"

#define UE_API COMMONCONVERSATIONRUNTIME_API

class UActorComponent;
class UConversationDatabase;
class UConversationInstance;
class UConversationParticipantComponent;
struct FConversationContext;

USTRUCT()
struct FConversationChoiceData
{
	GENERATED_BODY()

public:
	FConversationChoiceData() { }
	virtual ~FConversationChoiceData() = default;

	virtual UScriptStruct* GetScriptStruct() const
	{
		return FConversationChoiceData::StaticStruct();
	}
};

//template<>
//struct TStructOpsTypeTraits<FConversationChoiceData> : public TStructOpsTypeTraitsBase2<FConversationChoiceData>
//{
//	enum
//	{
//		WithNetSerializer = true	// For now this is REQUIRED for FConversationChoiceDataHandle net serialization to work
//	};
//};

USTRUCT()
struct FConversationChoiceDataHandle
{
	GENERATED_BODY()

public:
	/** Raw storage of target data, do not modify this directly */
	TArray<TSharedPtr<FConversationChoiceData>, TInlineAllocator<1> >	Data;

public:
	FConversationChoiceDataHandle() { }
	FConversationChoiceDataHandle(FConversationChoiceData* DataPtr)
	{
		Data.Add(TSharedPtr<FConversationChoiceData>(DataPtr));
	}

	FConversationChoiceDataHandle(FConversationChoiceDataHandle&& Other) : Data(MoveTemp(Other.Data)) { }
	FConversationChoiceDataHandle(const FConversationChoiceDataHandle& Other) : Data(Other.Data) { }

	FConversationChoiceDataHandle& operator=(FConversationChoiceDataHandle&& Other) { Data = MoveTemp(Other.Data); return *this; }
	FConversationChoiceDataHandle& operator=(const FConversationChoiceDataHandle& Other) { Data = Other.Data; return *this; }

	/** Resets handle to have no targets */
	void Clear()
	{
		Data.Reset();
	}

	/** Returns number of target data, not number of actors/targets as target data may contain multiple actors */
	int32 Num() const
	{
		return Data.Num();
	}

	/** Returns true if there are any valid targets */
	bool IsValid(int32 Index) const
	{
		return (Index < Data.Num() && Data[Index].IsValid());
	}

	/** Returns data at index, or nullptr if invalid */
	const FConversationChoiceData* Get(int32 Index) const
	{
		return IsValid(Index) ? Data[Index].Get() : nullptr;
	}

	/** Returns data at index, or nullptr if invalid */
	FConversationChoiceData* Get(int32 Index)
	{
		return IsValid(Index) ? Data[Index].Get() : nullptr;
	}

	/** Adds a new target data to handle, it must have been created with new */
	void Add(FConversationChoiceData* DataPtr)
	{
		Data.Add(TSharedPtr<FConversationChoiceData>(DataPtr));
	}

	/** Does a shallow copy of target data from one handle to another */
	void Append(const FConversationChoiceDataHandle& OtherHandle)
	{
		Data.Append(OtherHandle.Data);
	}

	/** Serialize for networking, handles polymorphism */
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Comparison operator */
	bool operator==(const FConversationChoiceDataHandle& Other) const
	{
		// Both invalid structs or both valid and Pointer compare (???) // deep comparison equality
		if (Data.Num() != Other.Data.Num())
		{
			return false;
		}
		for (int32 i = 0; i < Data.Num(); ++i)
		{
			if (Data[i].IsValid() != Other.Data[i].IsValid())
			{
				return false;
			}
			if (Data[i].Get() != Other.Data[i].Get())
			{
				return false;
			}
		}
		return true;
	}

	/** Comparison operator */
	bool operator!=(const FConversationChoiceDataHandle& Other) const
	{
		return !(FConversationChoiceDataHandle::operator==(Other));
	}
};

template<>
struct TStructOpsTypeTraits<FConversationChoiceDataHandle> : public TStructOpsTypeTraitsBase2<FConversationChoiceDataHandle>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FConversationChoiceData> Data is copied around
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
	};
};

/**
 * 
 */
USTRUCT(BlueprintType)
struct FConversationParticipantEntry
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	TObjectPtr<AActor> Actor = nullptr;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FGameplayTag ParticipantID;

	template<class TParticipantComponentClass = UConversationParticipantComponent>
	TParticipantComponentClass* GetParticipantComponent() const
	{
		return Actor ? Actor->FindComponentByClass<TParticipantComponentClass>() : nullptr;
	}
};

/**
 * 
 */
USTRUCT(BlueprintType)
struct FConversationParticipants
{
	GENERATED_BODY()

public:

	UE_API const FConversationParticipantEntry* GetParticipant(FGameplayTag ParticipantID) const;

	UE_API UConversationParticipantComponent* GetParticipantComponent(FGameplayTag ParticipantID) const;

	UE_API bool Contains(AActor* PotentialParticipant) const;

public:
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	TArray<FConversationParticipantEntry> List;
};

//////////////////////////////////////////////////////////////////////

USTRUCT(BlueprintType)
struct FConversationNodeParameterPair
{
	GENERATED_BODY()
public:
	FConversationNodeParameterPair() { }
	FConversationNodeParameterPair(const FString& InName, const FString& InValue) : Name(InName), Value(InValue) { }

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Conversation)
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Conversation)
	FString Value;
};

inline bool operator==(const FConversationNodeParameterPair& Lhs, const FConversationNodeParameterPair& Rhs) { return Lhs.Name == Rhs.Name && Lhs.Value == Rhs.Value; }
inline bool operator!=(const FConversationNodeParameterPair& Lhs, const FConversationNodeParameterPair& Rhs) { return !(Lhs == Rhs); }

/**
 * The conversation choice reference is the closest thing there is to a link at runtime for a choice. 
 * Choices always map to a Task node, which is the NodeReference.  However some tasks could potentially
 * dynamically generate several possible choices.  In that case they would do so with a unique set of
 * NodeParameters.  These would allow one task, say, "Sell N Items", where it's one task offering 3
 * different items for sale, and so it needs to match-up those choices with what the user chooses later.
 * the NodeParameters would contain the dynamic data generated at runtime signifying for the server
 * what to choose.
 *
 * Luckily the user does not need to guard against cheating clients.  Any dynamically generated choice
 * is remembered by the server - so any choice selected dynamically can be verified by the server as
 * one it offered to the user.
 */
USTRUCT(BlueprintType)
struct FConversationChoiceReference
 {
	GENERATED_BODY()

public:
	static UE_API const FConversationChoiceReference Empty;

	FConversationChoiceReference() {}
	FConversationChoiceReference(const FGuid& InNodeGUID) : NodeReference(InNodeGUID) { }

	/** This is the node that we're targeting with our choice. */
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FConversationNodeHandle NodeReference;

	/**
	 * These are the parameters required by that node to be activated.  The 
	 * same node could potentially handle several dynamic choices it provides, so these
     * parameters are used to uniquely identify the choice when the client responds to the server.
	 */
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	TArray<FConversationNodeParameterPair> NodeParameters;

	bool IsValid() const { return NodeReference.IsValid(); }

	UE_API FString ToString() const;
};

inline bool operator==(const FConversationChoiceReference& Lhs, const FConversationChoiceReference& Rhs) { return Lhs.NodeReference == Rhs.NodeReference && Lhs.NodeParameters == Rhs.NodeParameters; }
inline bool operator!=(const FConversationChoiceReference& Lhs, const FConversationChoiceReference& Rhs) { return !(Lhs == Rhs); }

//////////////////////////////////////////////////////////////////////

USTRUCT(BlueprintType)
struct FAdvanceConversationRequest
{
	GENERATED_BODY()

public:
	static UE_API const FAdvanceConversationRequest Any;

	FAdvanceConversationRequest() {}
	FAdvanceConversationRequest(const FConversationChoiceReference& InChoice) : Choice(InChoice) {}

public:
	UE_API FString ToString() const;

public:
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FConversationChoiceReference Choice;
	
	//@TODO: CONVERSATION: Not currently supported, TODO.
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	TArray<FConversationNodeParameterPair> UserParameters;
};

//////////////////////////////////////////////////////////////////////

/**
 *
 */
UENUM()
enum class EConversationChoiceType : uint8
{
	/**
	 * ServerOnly choices are the choices the user shouldn't see and are filtered from the client.
	 */
	ServerOnly,
	UserChoiceAvailable,
	UserChoiceUnavailable
};

/**
 * The conversation option entry is what we send to the client, one entry per choice.
 */
USTRUCT(BlueprintType)
struct FClientConversationOptionEntry
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FText ChoiceText;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FGameplayTagContainer ChoiceTags;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	EConversationChoiceType ChoiceType = EConversationChoiceType::ServerOnly;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FConversationChoiceReference ChoiceReference;

	/**
	 * Occasionally a choice might need to send down metadata that's entirely extra.
     * It's just bonus information for the client to do things like show more information
     * in the UI.  This information is not used on the return to the server to make the choice.
	 */
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	TArray<FConversationNodeParameterPair> ExtraData;

	/**
	 * Tries to resolve the node, this may fail, the guid might be bogus, or the node might not be
	 * in memory.
	 */
	template<class TConversationNodeClass = UConversationNode>
	const TConversationNodeClass* TryToResolveChoiceNode(const FConversationContext& Context) const
	{
		return Cast<TConversationNodeClass>(ChoiceReference.NodeReference.TryToResolve(Context));
	}

	template<class TConversationNodeClass = UConversationNode>
	const TConversationNodeClass* TryToResolveChoiceNode_Slow(UWorld* InWorld, const UConversationDatabase* Graph = nullptr) const
	{
		return Cast<TConversationNodeClass>(ChoiceReference.NodeReference.TryToResolve_Slow(InWorld, Graph));
	}

	void SetChoiceAvailable(bool bIsAvailable) { ChoiceType = bIsAvailable ? EConversationChoiceType::UserChoiceAvailable : EConversationChoiceType::UserChoiceUnavailable; }
	bool IsChoiceAvailable() const { return ChoiceType != EConversationChoiceType::UserChoiceUnavailable; }

	const FConversationChoiceReference& ToChoiceReference() const { return ChoiceReference; }
	UE_API FAdvanceConversationRequest ToAdvanceConversationRequest(const TArray<FConversationNodeParameterPair>& InUserParameters = TArray<FConversationNodeParameterPair>()) const;

	FString GetChoiceDataForKey(const FString& ParamKey) const
	{
		const FConversationNodeParameterPair* NodeParameter = ChoiceReference.NodeParameters.FindByPredicate([=](const FConversationNodeParameterPair& Pair) { return Pair.Name.Equals(ParamKey, ESearchCase::IgnoreCase); });
		if (NodeParameter)
		{
			return NodeParameter->Value;
		}

		const FConversationNodeParameterPair* ExtraDataParameter = ExtraData.FindByPredicate([=](const FConversationNodeParameterPair& Pair) { return Pair.Name.Equals(ParamKey, ESearchCase::IgnoreCase); });
		if (ExtraDataParameter)
		{
			return ExtraDataParameter->Value;
		}

		return TEXT("");
	}
};

inline bool operator==(const FClientConversationOptionEntry& Lhs, const FClientConversationOptionEntry& Rhs) { return Lhs.ChoiceText.IdenticalTo(Rhs.ChoiceText, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants) && Lhs.ChoiceTags == Rhs.ChoiceTags && Lhs.ChoiceType == Rhs.ChoiceType && Lhs.ChoiceReference == Rhs.ChoiceReference && Lhs.ExtraData == Rhs.ExtraData; }
inline bool operator!=(const FClientConversationOptionEntry& Lhs, const FClientConversationOptionEntry& Rhs) { return !(Lhs == Rhs); }

inline bool operator==(const FClientConversationOptionEntry& Lhs, const FConversationChoiceReference& Rhs) { return Lhs.ChoiceReference == Rhs; }
inline bool operator!=(const FClientConversationOptionEntry& Lhs, const FConversationChoiceReference& Rhs) { return !(Lhs == Rhs); }

//////////////////////////////////////////////////////////////////////

/**
 * You can think of the FConversationBranchPoint as the owner of FClientConversationOptionEntry.
 * We don't send this one to the client though, we store temporary state here so that when a user
 * picks a choice there may be other things we need to know or store for actions we need to take
 * due to the user picking this choice.  For example, some choices may introduce needing to
 * 'push' a new scope so that when that subtree terminates we return to the previous point, like
 * a stack.  There's no reason the client needs to understand this, so we just store it here.
 */
USTRUCT()
struct FConversationBranchPoint
{
	GENERATED_BODY()

public:
	FConversationBranchPoint() { }
	FConversationBranchPoint(const FClientConversationOptionEntry& InClientChoice) : ClientChoice(InClientChoice) { }
	FConversationBranchPoint(FClientConversationOptionEntry&& InClientChoice) : ClientChoice(MoveTemp(InClientChoice)) { }

	const FConversationNodeHandle& GetNodeHandle() const { return ClientChoice.ChoiceReference.NodeReference; }

	UPROPERTY()
	TArray<FConversationNodeHandle> ReturnScopeStack;

	UPROPERTY()
	FClientConversationOptionEntry ClientChoice;
};

//////////////////////////////////////////////////////////////////////

struct FConversationBranchPointBuilder : public FNoncopyable
{
	UE_API void AddChoice(const FConversationContext& InContext, FClientConversationOptionEntry&& InChoice);
	
	int32 Num() const { return BranchPoints.Num(); }
	const TArray<FConversationBranchPoint>& GetBranches() const { return BranchPoints; }

private:
	TArray<FConversationBranchPoint> BranchPoints;
};

//////////////////////////////////////////////////////////////////////

/**
 *
 */
USTRUCT(BlueprintType)
struct FClientConversationMessage
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FGameplayTag SpeakerID;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FText ParticipantDisplayName;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FText Text;

	// Additional metadata for this message (e.g. Emotion: "Angry"/"Happy")
	UPROPERTY(BlueprintReadWrite, Category = Conversation)
	TArray<FConversationNodeParameterPair> MetadataParameters;

	FString ToString() const { return Text.ToString(); }
};

//////////////////////////////////////////////////////////////////////

/**
 *
 */
USTRUCT(BlueprintType)
struct FClientConversationMessagePayload
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FClientConversationMessage Message;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FConversationParticipants Participants;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	FConversationNodeHandle CurrentNode;

	UPROPERTY(BlueprintReadWrite, Category=Conversation)
	TArray<FClientConversationOptionEntry> Options;

	FString ToString() const { return Message.ToString(); }
};

//////////////////////////////////////////////////////////////////////

#undef UE_API
